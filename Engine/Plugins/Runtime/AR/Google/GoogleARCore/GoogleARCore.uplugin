{
    "FileVersion" : 3,

    "Version" : 11,
    "VersionName" : "1.18.0",
    "FriendlyName" : "Google ARCore",
    "Description" : "Support for Google's AR platform.",
    "Category" : "Augmented Reality",
    "CreatedBy" : "Google",
    "CreatedByURL" : "https://developers.google.com/ar/",
    "CanContainContent": true,
    "IsBetaVersion": false,
    "EnabledByDefault": false,
	"SupportedTargetPlatforms": [
		"Android"
	],
    "Modules" :
    [
        {
            "Name" : "GoogleARCoreBase",
            "Type" : "Runtime",
            "PlatformAllowList": [ "Win64", "Mac", "Android", "Linux" ]
        },
        {
            "Name": "GoogleARCoreRendering",
            "Type": "Runtime",
            "LoadingPhase" : "PostConfigInit",
            "PlatformAllowList": [ "Win64", "Mac", "Android", "Linux" ]
        },
    ],
	"Plugins": [
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "AndroidPermission",
			"Enabled": true
		},
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		},
		{
			"Name": "ARUtilities",
			"Enabled": true
		}
	]
}
