// Copyright Epic Games, Inc. All Rights Reserved.


using System.IO;


namespace UnrealBuildTool.Rules
{
	public class GoogleARCoreBase : ModuleRules
	{
		public GoogleARCoreBase(ReadOnlyTargetRules Target) : base(Target)
		{
			CppCompileWarningSettings.DeterministicWarningLevel = WarningLevel.Off; // __DATE__ __TIME__ in Private/GoogleARCoreXRTrackingSystem.cpp

			PublicIncludePathModuleNames.Add("TargetPlatform");

			PublicDependencyModuleNames.AddRange(new string[]
			{
				"HeadMountedDisplay",
				"AugmentedReality",
			});

			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"EngineSettings",
				"Slate",
				"SlateCore",
				"RHI",
				"RenderCore",
				"AndroidPermission",
				"GoogleARCoreRendering",
				"GoogleARCoreSDK",
				"OpenGL",
				"ProceduralMeshComponent",
				"UElibPNG",
				"zlib",
				"MRMesh",
				"XRBase",
			});

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Settings" // For editor settings panel.
				}
			);

			if (Target.Platform == UnrealTargetPlatform.Android)
			{
				// Additional dependencies on android...
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						// Vulkan support for ARCore
						"Vulkan",
						"VulkanRHI",

						"Launch",
						"OpenGLDrv"
					}
				);

				// Register Plugin Language
				string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "GoogleARCoreBase_APL.xml"));
			}

			if (Target.bBuildEditor)
			{
				string ExecName = "";
				if (Target.Platform == UnrealTargetPlatform.Win64)
				{
					ExecName = "Windows/arcoreimg.exe";
				}
				else if (Target.Platform == UnrealTargetPlatform.Linux)
				{
					ExecName = "Linux/arcoreimg";
				}
				else if (Target.Platform == UnrealTargetPlatform.Mac)
				{
					ExecName = "Mac/ptdbtool_macos_lipobin";
				}
				
				if (ExecName.Length > 0)
				{
					RuntimeDependencies.Add("$(EngineDir)/Plugins/Runtime/AR/Google/GoogleARCore/Binaries/ThirdParty/Google/ARCoreImg/" + ExecName);
				}
			}
        }
    }
}
