<?xml version="1.0" encoding="utf-8"?>
<!--GoogleARCore plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
  <!-- init section is always evaluated once per architecture -->
  <init>
    <log text="GoogleARCorePlugin APL init"/>
	<!-- Get the bARCoreRequiredApp setting from the Engine ini. -->
	<setBoolFromProperty result="bARCoreRequiredApp" ini="Engine" section="/Script/GoogleARCoreBase.GoogleARCoreEditorSettings" property="bARCoreRequiredApp" default="true" />
  </init>

  <!-- optional files or directories to copy to Intermediate/Android/APK -->
  <resourceCopies>
    <log text="Copying GoogleARCore plugin files to staging"/>
    <copyDir src="$S(PluginDir)/Java" dst="$S(BuildDir)/src/com/google/arcore/unreal" />
  </resourceCopies>

  <!-- optional updates applied to AndroidManifest.xml -->
  <androidManifestUpdates>
    <addPermission android:name="android.permission.CAMERA"/>
    <if condition="bARCoreRequiredApp">
      <true>
        <addFeature android:name="android.hardware.camera.ar" android:required="true"/>
        <addElements tag="application">
          <meta-data android:name="com.google.ar.core" android:value="required" />
        </addElements>
      </true>
      <false>
        <addElements tag="application">
          <meta-data android:name="com.google.ar.core" android:value="optional" />
        </addElements>
      </false>
    </if>
  </androidManifestUpdates>


  <gameActivityImportAdditions>
    <insert>
			import com.google.arcore.unreal.GoogleARCoreJavaHelper;
		</insert>
  </gameActivityImportAdditions>

  <gameActivityClassAdditions>
    <insert>

      private GoogleARCoreJavaHelper arCoreJavaHelper = null;

      public GoogleARCoreJavaHelper GetGoogleARCoreJavaHelper()
      {
        if (arCoreJavaHelper == null)
        {
          arCoreJavaHelper = new GoogleARCoreJavaHelper(this);
        }

        return arCoreJavaHelper;
      }

      // Methods for calling via JNI:

      public int AndroidThunkJava_GetDisplayRotation()
      {
        return GetGoogleARCoreJavaHelper().getDisplayRotation();
      }

      public void AndroidThunkJava_QueueStartSessionOnUiThread()
      {
        GetGoogleARCoreJavaHelper().queueSessionStartOnUiThread();
      }
    </insert>
  </gameActivityClassAdditions>

  <gameActivityOnPauseAdditions>
    <insert>
      GoogleARCoreJavaHelper.onApplicationPause();
    </insert>
  </gameActivityOnPauseAdditions>

  <gameActivityOnResumeAdditions>
    <insert>
       GoogleARCoreJavaHelper.onApplicationResume();
    </insert>
  </gameActivityOnResumeAdditions>

  <gameActivityOnStartAdditions>
    <insert>
       GoogleARCoreJavaHelper.onApplicationStart();
     </insert>
  </gameActivityOnStartAdditions>

  <gameActivityOnStopAdditions>
    <insert>
       GoogleARCoreJavaHelper.onApplicationStop();
     </insert>
  </gameActivityOnStopAdditions>
  <gameActivityOnActivityResultAdditions>
    <insert>
    </insert>
  </gameActivityOnActivityResultAdditions>

  <gameActivityOnCreateAdditions>
    <insert>
      GoogleARCoreJavaHelper.onApplicationCreated();
      GetGoogleARCoreJavaHelper().initDisplayManager();
    </insert>
  </gameActivityOnCreateAdditions>

  <gameActivityOnDestroyAdditions>
    <insert>
      GoogleARCoreJavaHelper.onApplicationDestroyed();
    </insert>
  </gameActivityOnDestroyAdditions>

  <buildGradleAdditions>
      <insert>
        dependencies {
          implementation('com.google.ar:core:1.48.0')
        }
      </insert>
  </buildGradleAdditions>

</root>
