// Copyright Epic Games, Inc. All Rights Reserved.

package com.google.arcore.unreal;

import android.app.Activity;
import android.hardware.display.DisplayManager;
import android.view.Display;

public class GoogleARCoreJavaHelper {
    private Activity mParent;
    private DisplayManager displayManager;

    public GoogleARCoreJavaHelper(Activity unrealActivity) {
        mParent = unrealActivity;
    }

    public void initDisplayManager() {
        displayManager = (DisplayManager) mParent.getSystemService(mParent.DISPLAY_SERVICE);
        if(displayManager != null){
            displayManager.registerDisplayListener(new DisplayManager.DisplayListener() {
                @Override
                public void onDisplayAdded(int displayId) {
                }

                @Override
                public void onDisplayChanged(int displayId) {
                    synchronized (this) {
                        GoogleARCoreJavaHelper.onDisplayOrientationChanged();
                    }
                }

                @Override
                public void onDisplayRemoved(int displayId) {
                }
            }, null);
        }
    }

    public int getDisplayRotation() {
        Display display = mParent.getWindowManager().getDefaultDisplay();
        return display.getRotation();
    }

	public void queueSessionStartOnUiThread()
	{
		mParent.runOnUiThread (new Runnable() {
			@Override
			public void run() {
				ARCoreSessionStart();
			}
		});
	}

    public static native void onApplicationCreated();
    public static native void onApplicationDestroyed();
    public static native void onApplicationPause();
    public static native void onApplicationResume();
    public static native void onApplicationStop();
    public static native void onApplicationStart();
    public static native void onDisplayOrientationChanged();

	public static native void ARCoreSessionStart();
}
