// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreAPI.h"
#include "GoogleARCoreCameraImage.h"
#include "Misc/EngineVersion.h"
#include "DrawDebugHelpers.h"
#include "Math/NumericLimits.h"
#include "Templates/Casts.h"
#include "GoogleARCoreDevice.h"
#include "GoogleARCoreXRTrackingSystem.h"
#include "ARLifeCycleComponent.h"
#include "GoogleARCoreResources.h"
#include "GoogleARCoreBaseModule.h"
#include "GoogleARCoreTexture.h"

DEFINE_LOG_CATEGORY(LogGoogleARCoreAPI);

#if PLATFORM_ANDROID
#include "Android/AndroidApplication.h"
#include "Android/AndroidJNI.h"
#include "arcore_c_api.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Update AR Session"), STAT_UpdateARSession, STATGROUP_ARCore);
DECLARE_CYCLE_STAT(TEXT("Update AR Frame"), STAT_UpdateARFrame, STATGROUP_ARCore);
DECLARE_CYCLE_STAT(TEXT("Update AR Frame - Camera"), STAT_UpdateCamera, STATGROUP_ARCore);
DECLARE_CYCLE_STAT(TEXT("Update AR Frame - CameraImage"), STAT_UpdateCameraImage, STATGROUP_ARCore);
DECLARE_CYCLE_STAT(TEXT("Update AR Frame - DepthImage"), STAT_UpdateDepthImage, STATGROUP_ARCore);
DECLARE_CYCLE_STAT(TEXT("Update AR Frame - PointCloud"), STAT_UpdatePointCloud, STATGROUP_ARCore);
DECLARE_CYCLE_STAT(TEXT("Update AR Frame - Trackables"), STAT_UpdateTrackables, STATGROUP_ARCore);
DECLARE_CYCLE_STAT(TEXT("Update AR Frame - Anchors"), STAT_UpdateAnchors, STATGROUP_ARCore);

namespace
{
#if PLATFORM_ANDROID
	static const FMatrix ARCoreToUnrealTransform = FMatrix(
		FPlane(0.0f, 0.0f, -1.0f, 0.0f),
		FPlane(1.0f, 0.0f, 0.0f, 0.0f),
		FPlane(0.0f, 1.0f, 0.0f, 0.0f),
		FPlane(0.0f, 0.0f, 0.0f, 1.0f));

	static const FMatrix ARCoreToUnrealTransformInverse = ARCoreToUnrealTransform.InverseFast();

	EGoogleARCoreAPIStatus ToARCoreAPIStatus(ArStatus Status)
	{
		return static_cast<EGoogleARCoreAPIStatus>(Status);
	}

	FGoogleARCoreCameraConfig ToARCoreCameraConfig(const ArSession* SessionHandle, const ArCameraConfig* CameraConfigHandle)
	{
		FGoogleARCoreCameraConfig OutConfig;
		ArCameraConfig_getImageDimensions(SessionHandle, CameraConfigHandle, &OutConfig.CameraImageResolution.X, &OutConfig.CameraImageResolution.Y);
		ArCameraConfig_getTextureDimensions(SessionHandle, CameraConfigHandle, &OutConfig.CameraTextureResolution.X, &OutConfig.CameraTextureResolution.Y);
		char* CameraID = nullptr;
		ArCameraConfig_getCameraId(SessionHandle, CameraConfigHandle, &CameraID);
		OutConfig.CameraID = FString(ANSI_TO_TCHAR(CameraID));

		// Figure out depth sensor usage mode
		uint32_t DepthSensorUsage = 0;
		ArCameraConfig_getDepthSensorUsage(SessionHandle, CameraConfigHandle, &DepthSensorUsage);
		OutConfig.DepthSensorUsage = DepthSensorUsage;

		// Figure out target FPS mode
		int32_t MinFPS = 0;
		int32_t MaxFPS = 0;
		ArCameraConfig_getFpsRange(SessionHandle, CameraConfigHandle, &MinFPS, &MaxFPS);
		OutConfig.SetMaxFPS(MaxFPS);

		// Figure out the camera facing
		ArCameraConfigFacingDirection FacingDirection = AR_CAMERA_CONFIG_FACING_DIRECTION_BACK;
		ArCameraConfig_getFacingDirection(SessionHandle, CameraConfigHandle, &FacingDirection);
		OutConfig.CameraFacing = FacingDirection == AR_CAMERA_CONFIG_FACING_DIRECTION_BACK ? EGoogleARCoreCameraFacing::Back : EGoogleARCoreCameraFacing::Front;

		return OutConfig;
	}

	void UnrealTransformToARCorePose(const FTransform& UnrealTransform, const ArSession* SessionHandle, ArPose** OutARPose, float WorldToMeterScale)
	{
		check(OutARPose);

		FMatrix UnrealPoseMatrix = UnrealTransform.ToMatrixNoScale();
		UnrealPoseMatrix.SetOrigin(UnrealPoseMatrix.GetOrigin() / WorldToMeterScale);
		FMatrix ARCorePoseMatrix = ARCoreToUnrealTransformInverse * UnrealPoseMatrix * ARCoreToUnrealTransform;

		FVector ArPosePosition = ARCorePoseMatrix.GetOrigin();
		FQuat ArPoseRotation = ARCorePoseMatrix.ToQuat();
		float ArPoseData[7] = { (float)ArPoseRotation.X, (float)ArPoseRotation.Y, (float)ArPoseRotation.Z, (float)ArPoseRotation.W, (float)ArPosePosition.X, (float)ArPosePosition.Y, (float)ArPosePosition.Z };	// LWC_TODO: Precision loss.
		ArPose_create(SessionHandle, ArPoseData, OutARPose);
	}

	FVector UnrealPositionToARCorePosition(const FVector& UnrealPosition, float WorldToMeterScale)
	{
		FVector Result = ARCoreToUnrealTransform.TransformPosition(UnrealPosition / WorldToMeterScale);
		return Result;
	}

	EGoogleARCoreAPIStatus DeserializeAugmentedImageDatabase(const ArSession* SessionHandle, const TArray<uint8>& SerializedDatabase, ArAugmentedImageDatabase*& DatabaseNativeHandle)
	{
		ArAugmentedImageDatabase* AugmentedImageDb = nullptr;

		if (SerializedDatabase.Num() == 0)
		{
			UE_LOGF(LogGoogleARCoreAPI, Error, "AugmentedImageDatabase contains no cooked data! The cooking process for AugmentedImageDatabase may have failed. Check the Unreal Editor build log for details.");
			return EGoogleARCoreAPIStatus::AR_ERROR_DATA_INVALID_FORMAT;
		}

		EGoogleARCoreAPIStatus Status = ToARCoreAPIStatus(
			ArAugmentedImageDatabase_deserialize(
				SessionHandle, &SerializedDatabase[0],
				SerializedDatabase.Num(),
				&AugmentedImageDb));

		if (Status != EGoogleARCoreAPIStatus::AR_SUCCESS)
		{
			UE_LOGF(LogGoogleARCoreAPI, Error, "ArAugmentedImageDatabase_deserialize failed!");
			return Status;
		}

		DatabaseNativeHandle = AugmentedImageDb;
		return Status;
	}

	ArCoordinates2dType ToArCoordinates2dType(EGoogleARCoreCoordinates2DType UnrealType)
	{
		switch (UnrealType)
		{
		case EGoogleARCoreCoordinates2DType::Image:
			return AR_COORDINATES_2D_IMAGE_NORMALIZED;
		case EGoogleARCoreCoordinates2DType::Texture:
			return AR_COORDINATES_2D_TEXTURE_NORMALIZED;
		case EGoogleARCoreCoordinates2DType::Viewport:
			return AR_COORDINATES_2D_VIEW_NORMALIZED;
		default:
			UE_LOGF(LogGoogleARCoreAPI, Error, "Unknown conversion from EGoogleARCoreCoordinates2DType to ArCoordinates2dType.");
			return AR_COORDINATES_2D_VIEW_NORMALIZED;
		}
	}
#endif
}

bool CheckIsSessionValid(FString TypeName, const TWeakPtr<FGoogleARCoreSession>& SessionPtr)
{
	if (!SessionPtr.IsValid())
	{
		return false;
	}
#if PLATFORM_ANDROID
	if (SessionPtr.Pin()->GetHandle() == nullptr)
	{
		return false;
	}
#endif
	return true;
}

#if PLATFORM_ANDROID
EARTrackingState ToARTrackingState(ArTrackingState State)
{
	switch (State)
	{
	case AR_TRACKING_STATE_PAUSED:
		return EARTrackingState::NotTracking;
	case AR_TRACKING_STATE_STOPPED:
		return EARTrackingState::StoppedTracking;
	case AR_TRACKING_STATE_TRACKING:
		return EARTrackingState::Tracking;
	}
}

FTransform ARCorePoseToUnrealTransform(ArPose* ArPoseHandle, const ArSession* SessionHandle, float WorldToMeterScale)
{
	FMatrix44f ARCorePoseMatrix;
	ArPose_getMatrix(SessionHandle, ArPoseHandle, ARCorePoseMatrix.M[0]);
	FTransform Result = FTransform(ARCoreToUnrealTransform * FMatrix(ARCorePoseMatrix) * ARCoreToUnrealTransformInverse);
	Result.SetLocation(Result.GetLocation() * WorldToMeterScale);

	return Result;
}
#endif

extern "C"
{
#if PLATFORM_ANDROID
void ArSession_reportEngineType(ArSession* session, const char* engine_type, const char* engine_version);
#endif
}

/****************************************/
/*       FGoogleARCoreAPKManager        */
/****************************************/
EGoogleARCoreAvailability FGoogleARCoreAPKManager::CheckARCoreAPKAvailability()
{
#if PLATFORM_ANDROID
	static JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "getApplicationContext", "()Landroid/content/Context;", false);
	static jobject ApplicationContext = FJavaWrapper::CallObjectMethod(Env, FAndroidApplication::GetGameActivityThis(), Method);
	
	// OutAvailability is cached here as calling 'ArCoreApk_checkAvailability' can cause memory leak in JAVA on some devices
	// so we bail out immediately if the last call returns OK (this of course assumes the availability doesn't degrades during runtime)
	static ArAvailability OutAvailability = AR_AVAILABILITY_UNKNOWN_ERROR;
	if (OutAvailability == AR_AVAILABILITY_SUPPORTED_INSTALLED)
	{
		return EGoogleARCoreAvailability::SupportedInstalled;
	}
	
	ArCoreApk_checkAvailability(Env, ApplicationContext, &OutAvailability);
	
	// Use static_cast here since we already make sure the enum has the same value.
	return static_cast<EGoogleARCoreAvailability>(OutAvailability);
#else
	return EGoogleARCoreAvailability::UnsupportedDeviceNotCapable;
#endif
}

EGoogleARCoreAPIStatus FGoogleARCoreAPKManager::RequestInstall(bool bUserRequestedInstall, EGoogleARCoreInstallStatus& OutInstallStatus)
{
	EGoogleARCoreAPIStatus Status = EGoogleARCoreAPIStatus::AR_ERROR_FATAL;
#if PLATFORM_ANDROID
	static JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	static jobject ApplicationActivity = FAndroidApplication::GetGameActivityThis();

	ArInstallStatus OutAvailability = AR_INSTALL_STATUS_INSTALLED;
	Status = ToARCoreAPIStatus(ArCoreApk_requestInstall(Env, ApplicationActivity, bUserRequestedInstall, &OutAvailability));
	OutInstallStatus = static_cast<EGoogleARCoreInstallStatus>(OutAvailability);
#endif
	return Status;
}

/****************************************/
/*         FGoogleARCoreSession         */
/****************************************/
FGoogleARCoreSession::FGoogleARCoreSession()
	: SessionCreateStatus(EGoogleARCoreAPIStatus::AR_UNAVAILABLE_DEVICE_NOT_COMPATIBLE)
	, SessionConfig(nullptr)
	, LatestFrame(nullptr)
	, UObjectManager(nullptr)
	, CachedWorldToMeterScale(100.0f)
	, FrameNumber(0)

{
	// Create Android ARSession handle.
	LatestFrame = new FGoogleARCoreFrame(this);
#if PLATFORM_ANDROID
	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "getApplicationContext", "()Landroid/content/Context;", false);
	jobject ApplicationContext = FJavaWrapper::CallObjectMethod(Env, FAndroidApplication::GetGameActivityThis(), Method);
	check(Env);
	check(ApplicationContext);

	SessionCreateStatus = ToARCoreAPIStatus(ArSession_create(Env, ApplicationContext, &SessionHandle));

	if (SessionCreateStatus != EGoogleARCoreAPIStatus::AR_SUCCESS)
	{
		UE_LOGF(LogGoogleARCoreAPI, Error, "ArSession_create returns with error: %d", static_cast<int>(SessionCreateStatus));
		return;
	}

	ArConfig_create(SessionHandle, &ConfigHandle);
	LatestFrame->Init();

	static bool ARCoreAnalyticsReported = false;
	if (!ARCoreAnalyticsReported)
	{
		ArSession_reportEngineType(SessionHandle, "Unreal Engine", TCHAR_TO_ANSI(*FEngineVersion::Current().ToString()));
		ARCoreAnalyticsReported = true;
	}
#endif
}

FGoogleARCoreSession::~FGoogleARCoreSession()
{
	if (UObjectManager)
	{
		UObjectManager->ClearTrackables();
		for (auto Itr : UObjectManager->HandleToAnchorMap)
		{
			if (auto Anchor = Itr.Value)
			{
				Anchor->OnTrackingStateChanged(EARTrackingState::StoppedTracking);
			}
		}
	}
	
	delete LatestFrame;

#if PLATFORM_ANDROID
	if (SessionHandle != nullptr)
	{
		ArSession_destroy(SessionHandle);
		ArConfig_destroy(ConfigHandle);
	}
#endif
}

// Properties
EGoogleARCoreAPIStatus FGoogleARCoreSession::GetSessionCreateStatus()
{
	return SessionCreateStatus;
}

UGoogleARCoreUObjectManager* FGoogleARCoreSession::GetUObjectManager()
{
	return UObjectManager;
}

float FGoogleARCoreSession::GetWorldToMeterScale()
{
	return CachedWorldToMeterScale;
}
#if PLATFORM_ANDROID
ArSession* FGoogleARCoreSession::GetHandle()
{
	return SessionHandle;
}
#endif

// Session lifecycle
bool FGoogleARCoreSession::IsConfigSupported(const UARSessionConfig& Config)
{
#if PLATFORM_ANDROID
	// Always return true for now since all configuration is supported on all ARCore supported phones.
	return true;
#else
	return false;
#endif
}

EGoogleARCoreAPIStatus FGoogleARCoreSession::ConfigSession(const UARSessionConfig& Config)
{
	SessionConfig = &Config;
	EGoogleARCoreAPIStatus ConfigStatus = EGoogleARCoreAPIStatus::AR_SUCCESS;
	const UGoogleARCoreSessionConfig *GoogleConfig = Cast<UGoogleARCoreSessionConfig>(&Config);

#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return EGoogleARCoreAPIStatus::AR_ERROR_FATAL;
	}
	ArConfig_setLightEstimationMode(SessionHandle, ConfigHandle, static_cast<ArLightEstimationMode>(Config.GetLightEstimationMode()));
	ArPlaneFindingMode PlaneFindingMode = AR_PLANE_FINDING_MODE_DISABLED;
	EARPlaneDetectionMode PlaneMode = Config.GetPlaneDetectionMode();
	bool bHorizontalPlaneDetection = !!(PlaneMode & EARPlaneDetectionMode::HorizontalPlaneDetection);
	bool bVerticalPlaneDetection = !!(PlaneMode & EARPlaneDetectionMode::VerticalPlaneDetection);
	if (bHorizontalPlaneDetection && bVerticalPlaneDetection)
	{
		PlaneFindingMode = AR_PLANE_FINDING_MODE_HORIZONTAL_AND_VERTICAL;
	}
	else if (bHorizontalPlaneDetection)
	{
		PlaneFindingMode = AR_PLANE_FINDING_MODE_HORIZONTAL;
	}
	else if (bVerticalPlaneDetection)
	{
		PlaneFindingMode = AR_PLANE_FINDING_MODE_VERTICAL;
	}

	ArFocusMode FocusMode = Config.ShouldEnableAutoFocus() ? AR_FOCUS_MODE_AUTO : AR_FOCUS_MODE_FIXED;
	ArConfig_setPlaneFindingMode(SessionHandle, ConfigHandle, PlaneFindingMode);
	ArConfig_setUpdateMode(SessionHandle, ConfigHandle, static_cast<ArUpdateMode>(Config.GetFrameSyncMode()));
	ArConfig_setFocusMode(SessionHandle, ConfigHandle, FocusMode);

	static ArAugmentedImageDatabase* EmptyImageDatabaseHandle = nullptr;

	if (EmptyImageDatabaseHandle == nullptr)
	{
		ArAugmentedImageDatabase_create(SessionHandle, &EmptyImageDatabaseHandle);
	}

	ArConfig_setAugmentedImageDatabase(SessionHandle, ConfigHandle, EmptyImageDatabaseHandle);

	// If the candidate image list is set on the base config object, ignore the AugmentedImageDatabase since it is getting deprecated.
	if (GoogleConfig && GoogleConfig->AugmentedImageDatabase && Config.GetCandidateImageList().Num() == 0)
	{
		if (GoogleConfig->AugmentedImageDatabase->NativeHandle == nullptr && GoogleConfig->AugmentedImageDatabase->Entries.Num() != 0)
		{
			ConfigStatus = DeserializeAugmentedImageDatabase(SessionHandle, GoogleConfig->AugmentedImageDatabase->SerializedDatabase,
				GoogleConfig->AugmentedImageDatabase->NativeHandle);

			if (ConfigStatus != EGoogleARCoreAPIStatus::AR_SUCCESS)
			{
				return ConfigStatus;
			}
		}

		if (GoogleConfig->AugmentedImageDatabase->NativeHandle != nullptr)
		{
			ArConfig_setAugmentedImageDatabase(SessionHandle, ConfigHandle, GoogleConfig->AugmentedImageDatabase->NativeHandle);
		}
	}
	else if(GoogleConfig == nullptr && Config.GetCandidateImageList().Num() != 0)
	{
		ArAugmentedImageDatabase* AugmentedImageDb = nullptr;

		if (!ImageDatabaseMap.Contains(&Config))
		{
			ConfigStatus = DeserializeAugmentedImageDatabase(SessionHandle, Config.GetSerializedARCandidateImageDatabase(), AugmentedImageDb);

			if (ConfigStatus != EGoogleARCoreAPIStatus::AR_SUCCESS)
			{
				return ConfigStatus;
			}

			ImageDatabaseMap.Add(&Config, AugmentedImageDb);
		}
		else
		{
			AugmentedImageDb = ImageDatabaseMap[&Config];
		}
		ArConfig_setAugmentedImageDatabase(SessionHandle, ConfigHandle, AugmentedImageDb);
	}

	// Config Augmented Face
	ArAugmentedFaceMode FaceMode = AR_AUGMENTED_FACE_MODE_DISABLED;
	if (GoogleConfig != nullptr)
	{
		switch (GoogleConfig->AugmentedFaceMode)
		{
		case EGoogleARCoreAugmentedFaceMode::PoseAndMesh:
			FaceMode = AR_AUGMENTED_FACE_MODE_MESH3D;
			break;
		}
	}
	else if(Config.GetSessionType() == EARSessionType::Face)
	{
		FaceMode = AR_AUGMENTED_FACE_MODE_MESH3D;
	}
	ArConfig_setAugmentedFaceMode(SessionHandle, ConfigHandle, FaceMode);
	
	// Whether scene depth is requested and supported
	auto bShouldEnableSceneDepth = false;
	if (Config.GetEnabledSessionTrackingFeature() == EARSessionTrackingFeature::SceneDepth)
	{
		int32_t bSupported = 0;
		ArSession_isDepthModeSupported(SessionHandle, AR_DEPTH_MODE_AUTOMATIC, &bSupported);
		if (bSupported)
		{
			bShouldEnableSceneDepth = true;
		}
	}
	
	ArConfig_setDepthMode(SessionHandle, ConfigHandle, bShouldEnableSceneDepth ? AR_DEPTH_MODE_AUTOMATIC : AR_DEPTH_MODE_DISABLED);

	// Enabling mode to expose Hardware Buffer in Vulkan
	if (FAndroidMisc::ShouldUseVulkan())
	{
		ArConfig_setTextureUpdateMode(SessionHandle, ConfigHandle, AR_TEXTURE_UPDATE_MODE_EXPOSE_HARDWARE_BUFFER);
	}

	ConfigStatus = ToARCoreAPIStatus(ArSession_configure(SessionHandle, ConfigHandle));
#endif
	return ConfigStatus;
}

const UARSessionConfig* FGoogleARCoreSession::GetCurrentSessionConfig()
{
	return SessionConfig;
}

TArray<FGoogleARCoreCameraConfig> FGoogleARCoreSession::GetSupportedCameraConfig(EGoogleARCoreCameraFacing CameraFacing)
{
	TArray<FGoogleARCoreCameraConfig> SupportedConfigs;
#if PLATFORM_ANDROID
	ArCameraConfigList* CameraConfigList = nullptr;
	ArCameraConfigList_create(SessionHandle, &CameraConfigList);

	ArCameraConfigFilter* CameraConfigFilter = nullptr;
	ArCameraConfigFilter_create(SessionHandle, &CameraConfigFilter);

	// We must choose one facing or the other.
	ArCameraConfigFacingDirection Dir = CameraFacing == EGoogleARCoreCameraFacing::Front ? AR_CAMERA_CONFIG_FACING_DIRECTION_FRONT : AR_CAMERA_CONFIG_FACING_DIRECTION_BACK;
	ArCameraConfigFilter_setFacingDirection(SessionHandle, CameraConfigFilter, Dir);

	ArSession_getSupportedCameraConfigsWithFilter(SessionHandle, CameraConfigFilter, CameraConfigList);

	ArCameraConfig* CameraConfigHandle = nullptr;
	ArCameraConfig_create(SessionHandle, &CameraConfigHandle);

	int ListSize = 0;
	ArCameraConfigList_getSize(SessionHandle, CameraConfigList, &ListSize);

	for (int i = 0; i < ListSize; i++)
	{
		ArCameraConfigList_getItem(SessionHandle, CameraConfigList, i, CameraConfigHandle);
		FGoogleARCoreCameraConfig CameraConfig = ToARCoreCameraConfig(SessionHandle, CameraConfigHandle);
		SupportedConfigs.Add(CameraConfig);		
	}

	ArCameraConfig_destroy(CameraConfigHandle);
	CameraConfigHandle = nullptr;

	ArCameraConfigList_destroy(CameraConfigList);
	CameraConfigList = nullptr;

	ArCameraConfigFilter_destroy(CameraConfigFilter);
	CameraConfigFilter = nullptr;
#endif

	return SupportedConfigs;
}

EGoogleARCoreAPIStatus FGoogleARCoreSession::SetCameraConfig(FGoogleARCoreCameraConfig SelectedCameraConfig)
{
	UE_LOGF(LogGoogleARCoreAPI, Log, "SetCameraConfig");

#if PLATFORM_ANDROID
	ArCameraConfigList* CameraConfigList = nullptr;
	ArCameraConfigList_create(SessionHandle, &CameraConfigList);

	ArCameraConfigFilter* CameraConfigFilter = nullptr;
	ArCameraConfigFilter_create(SessionHandle, &CameraConfigFilter);

	// Filter on camera FPS
	ArCameraConfigFilter_setTargetFps(SessionHandle, CameraConfigFilter, SelectedCameraConfig.TargetFPS);

	// Filter on depth sensor usage
	ArCameraConfigFilter_setDepthSensorUsage(SessionHandle, CameraConfigFilter, SelectedCameraConfig.DepthSensorUsage);

	// Filter on facing
	if (SelectedCameraConfig.CameraFacing != EGoogleARCoreCameraFacing::None)
	{
		UE_LOGF(LogGoogleARCoreAPI, Log, "SelectedCameraConfig.CameraFacing %i", (int32)SelectedCameraConfig.CameraFacing);
		ArCameraConfigFilter_setFacingDirection(SessionHandle, CameraConfigFilter, SelectedCameraConfig.CameraFacing == EGoogleARCoreCameraFacing::Back ? AR_CAMERA_CONFIG_FACING_DIRECTION_BACK : AR_CAMERA_CONFIG_FACING_DIRECTION_FRONT);
	}

	ArSession_getSupportedCameraConfigsWithFilter(SessionHandle, CameraConfigFilter, CameraConfigList);

	int ListSize = 0;
	ArCameraConfigList_getSize(SessionHandle, CameraConfigList, &ListSize);
	UE_LOGF(LogGoogleARCoreAPI, Log, "ArCameraConfigList_getSize %i", ListSize);

	ArCameraConfig* CameraConfigHandle = nullptr;
	ArCameraConfig_create(SessionHandle, &CameraConfigHandle);

	ArStatus Status = AR_ERROR_INVALID_ARGUMENT;
	bool bFoundSelectedConfig = false;
	for (int i = 0; i < ListSize; i++)
	{
		ArCameraConfigList_getItem(SessionHandle, CameraConfigList, i, CameraConfigHandle);
		FGoogleARCoreCameraConfig CameraConfig = ToARCoreCameraConfig(SessionHandle, CameraConfigHandle);
		if (CameraConfig.IsCompatibleWith(SelectedCameraConfig))
		{
			Status = ArSession_setCameraConfig(SessionHandle, CameraConfigHandle);
			UE_LOGF(LogGoogleARCoreAPI, Log, "Configure ARCore session with camera config(Camera Image - %d x %d, Camera Texture - %d x %d) returns %d",
				CameraConfig.CameraImageResolution.X, CameraConfig.CameraImageResolution.Y,
				CameraConfig.CameraTextureResolution.X, CameraConfig.CameraTextureResolution.Y,
				(int)Status);
			bFoundSelectedConfig = true;
			break;
		}
		else
		{
			UE_LOGF(LogGoogleARCoreAPI, Log, "Configure ARCore session skipping incompatibile format(Camera Image - %d x %d, Camera Texture - %d x %d) returns %d",
				CameraConfig.CameraImageResolution.X, CameraConfig.CameraImageResolution.Y,
				CameraConfig.CameraTextureResolution.X, CameraConfig.CameraTextureResolution.Y,
				(int)Status);
		}
	}

	ArCameraConfig_destroy(CameraConfigHandle);
	CameraConfigHandle = nullptr;

	ArCameraConfigList_destroy(CameraConfigList);
	CameraConfigList = nullptr;

	ArCameraConfigFilter_destroy(CameraConfigFilter);
	CameraConfigFilter = nullptr;

	if (!bFoundSelectedConfig)
	{
		UE_LOGF(LogGoogleARCoreAPI, Error, "The provided CameraConfig isn't supported on this device!");
	}

	return ToARCoreAPIStatus(Status);
#else
	return EGoogleARCoreAPIStatus::AR_SUCCESS;
#endif
}

void FGoogleARCoreSession::GetARCameraConfig(FGoogleARCoreCameraConfig& OutCurrentCameraConfig)
{
#if PLATFORM_ANDROID
	ArCameraConfig* CameraConfigHandle = nullptr;
	ArCameraConfig_create(SessionHandle, &CameraConfigHandle);

	ArSession_getCameraConfig(SessionHandle, CameraConfigHandle);
	OutCurrentCameraConfig = ToARCoreCameraConfig(SessionHandle, CameraConfigHandle);

	ArCameraConfig_destroy(CameraConfigHandle);
#endif
}

int FGoogleARCoreSession::AddRuntimeAugmentedImage(UGoogleARCoreAugmentedImageDatabase* TargetImageDatabase, const TArray<uint8>& ImageGrayscalePixels,
	int ImageWidth, int ImageHeight, FString ImageName, float ImageWidthInMeter)
{
	int OutIndex = -1;
	ensure(TargetImageDatabase != nullptr);

#if PLATFORM_ANDROID
	if (TargetImageDatabase->NativeHandle == nullptr)
	{
		if (TargetImageDatabase->Entries.Num() != 0) {
			if (DeserializeAugmentedImageDatabase(SessionHandle, TargetImageDatabase->SerializedDatabase, TargetImageDatabase->NativeHandle)
				!= EGoogleARCoreAPIStatus::AR_SUCCESS)
			{
				UE_LOGF(LogGoogleARCoreAPI, Warning, "Failed to add runtime augmented image: AugmentedImageDatabase is corrupte.");
				return -1;
			}
		}
		else
		{
			ArAugmentedImageDatabase_create(SessionHandle, &TargetImageDatabase->NativeHandle);
		}
	}

	ArStatus Status = AR_SUCCESS;
	if (ImageWidthInMeter <= 0)
	{
		Status = ArAugmentedImageDatabase_addImage(SessionHandle, TargetImageDatabase->NativeHandle, TCHAR_TO_ANSI(*ImageName),
			ImageGrayscalePixels.GetData(), ImageWidth, ImageHeight, ImageWidth, &OutIndex);
	}
	else
	{
		Status = ArAugmentedImageDatabase_addImageWithPhysicalSize(SessionHandle, TargetImageDatabase->NativeHandle, TCHAR_TO_ANSI(*ImageName),
			ImageGrayscalePixels.GetData(), ImageWidth, ImageHeight, ImageWidth, ImageWidthInMeter, &OutIndex);
	}

	if (Status != AR_SUCCESS)
	{
		UE_LOGF(LogGoogleARCoreAPI, Warning, "Failed to add runtime augmented image: image quality is insufficient. %d", static_cast<int>(Status));
		return -1;
	}
#endif
	return OutIndex;
}

bool FGoogleARCoreSession::AddRuntimeCandidateImage(UARSessionConfig* TargetSessionConfig, const TArray<uint8>& ImageGrayscalePixels, int ImageWidth, int ImageHeight, FString FriendlyName, float PhysicsWidth)
{
#if PLATFORM_ANDROID
	ArAugmentedImageDatabase* DatabaseHandle = nullptr;
	if (!ImageDatabaseMap.Contains(TargetSessionConfig))
	{
		if (TargetSessionConfig->GetCandidateImageList().Num() != 0) {
			if (DeserializeAugmentedImageDatabase(SessionHandle, TargetSessionConfig->GetSerializedARCandidateImageDatabase(), DatabaseHandle) != EGoogleARCoreAPIStatus::AR_SUCCESS)
			{
				UE_LOGF(LogGoogleARCoreAPI, Warning, "Failed to add runtime augmented image: AugmentedImageDatabase is corrupte.");
				return false;
			}
		}
		else
		{
			ArAugmentedImageDatabase_create(SessionHandle, &DatabaseHandle);
		}

		ImageDatabaseMap.Add(TargetSessionConfig, DatabaseHandle);
	}
	else
	{
		DatabaseHandle = ImageDatabaseMap[TargetSessionConfig];
	}

	ArStatus Status = AR_SUCCESS;
	int OutIndex = 0;
	if (PhysicsWidth <= 0)
	{
		Status = ArAugmentedImageDatabase_addImage(SessionHandle, DatabaseHandle, TCHAR_TO_ANSI(*FriendlyName),
			ImageGrayscalePixels.GetData(), ImageWidth, ImageHeight, ImageWidth, &OutIndex);
	}
	else
	{
		Status = ArAugmentedImageDatabase_addImageWithPhysicalSize(SessionHandle, DatabaseHandle, TCHAR_TO_ANSI(*FriendlyName),
			ImageGrayscalePixels.GetData(), ImageWidth, ImageHeight, ImageWidth, PhysicsWidth, &OutIndex);
	}

	if (Status != AR_SUCCESS)
	{
		UE_LOGF(LogGoogleARCoreAPI, Warning, "Failed to add runtime augmented image: image quality is insufficient. %d", static_cast<int>(Status));
		return false;
	}
	return true;
#else
	return false;
#endif
}

EGoogleARCoreAPIStatus FGoogleARCoreSession::Resume()
{
	EGoogleARCoreAPIStatus ResumeStatus = EGoogleARCoreAPIStatus::AR_SUCCESS;
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return EGoogleARCoreAPIStatus::AR_ERROR_FATAL;
	}

	ResumeStatus = ToARCoreAPIStatus(ArSession_resume(SessionHandle));
#endif
	return ResumeStatus;
}

EGoogleARCoreAPIStatus FGoogleARCoreSession::Pause()
{
	EGoogleARCoreAPIStatus PauseStatue = EGoogleARCoreAPIStatus::AR_SUCCESS;
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return EGoogleARCoreAPIStatus::AR_ERROR_FATAL;
	}

	PauseStatue = ToARCoreAPIStatus(ArSession_pause(SessionHandle));

	// Update all tracked geometry tracking state.
	TArray<UARTrackedGeometry*> AllTrackedGeometries;
	GetAllTrackables<UARTrackedGeometry>(AllTrackedGeometries);
	for (UARTrackedGeometry* Trackable : AllTrackedGeometries)
	{
		if (Trackable->GetTrackingState() == EARTrackingState::Tracking)
		{
			Trackable->UpdateTrackingState(EARTrackingState::NotTracking);
		}
	}
#endif

	for (auto Itr : UObjectManager->HandleToAnchorMap)
	{
		if (auto Anchor = Itr.Value)
		{
			if (Anchor->GetTrackingState() == EARTrackingState::Tracking)
			{
				Anchor->OnTrackingStateChanged(EARTrackingState::NotTracking);
			}
		}
	}

	return PauseStatue;
}

EGoogleARCoreAPIStatus FGoogleARCoreSession::Update(float WorldToMeterScale)
{
	EGoogleARCoreAPIStatus UpdateStatus = EGoogleARCoreAPIStatus::AR_SUCCESS;

#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return EGoogleARCoreAPIStatus::AR_ERROR_FATAL;
	}
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateARSession);
		UpdateStatus = ToARCoreAPIStatus(ArSession_update(SessionHandle, LatestFrame->FrameHandle));
	}
#endif

	CachedWorldToMeterScale = WorldToMeterScale;
	int64 LastFrameTimestamp = LatestFrame->GetCameraTimestamp();
	LatestFrame->Update(WorldToMeterScale);
	if (LastFrameTimestamp != LatestFrame->GetCameraTimestamp())
	{
		FrameNumber++;
	}

	// If the timestamp is the same then the frame contains the same data.
	bHasNewFrame = (LastFrameTimestamp != LatestFrame->GetCameraTimestamp());

	return UpdateStatus;
}

const FGoogleARCoreFrame* FGoogleARCoreSession::GetLatestFrame()
{
	return LatestFrame;
}

uint32 FGoogleARCoreSession::GetFrameNum()
{
	return FrameNumber;
}

void FGoogleARCoreSession::SetCameraTextureIds(const TArray<uint32_t>& TextureIds)
{
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return;
	}
	// Camera textures names are not used with Vulkan since it relies on getting Hardware Buffer instead.
	if (!FAndroidMisc::ShouldUseVulkan())
	{
		ArSession_setCameraTextureNames(SessionHandle, TextureIds.Num(), TextureIds.GetData());
	}
#endif
}

void FGoogleARCoreSession::SetDisplayGeometry(int Rotation, int Width, int Height)
{
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return;
	}
	ArSession_setDisplayGeometry(SessionHandle, Rotation, Width, Height);
#endif
}

// Anchors and Planes
EGoogleARCoreAPIStatus FGoogleARCoreSession::CreateARAnchor(const FTransform& TransfromInTrackingSpace, UARTrackedGeometry* TrackedGeometry, USceneComponent* ComponentToPin, FName InDebugName, UARPin*& OutAnchor)
{
	EGoogleARCoreAPIStatus AnchorCreateStatus = EGoogleARCoreAPIStatus::AR_SUCCESS;
	OutAnchor = nullptr;

#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return EGoogleARCoreAPIStatus::AR_ERROR_SESSION_PAUSED;
	}

	ArAnchor* NewAnchorHandle = nullptr;
	ArPose *PoseHandle = nullptr;
	ArPose_create(SessionHandle, nullptr, &PoseHandle);
	UnrealTransformToARCorePose(TransfromInTrackingSpace, SessionHandle, &PoseHandle, CachedWorldToMeterScale);
	if (TrackedGeometry == nullptr)
	{
		AnchorCreateStatus = ToARCoreAPIStatus(ArSession_acquireNewAnchor(SessionHandle, PoseHandle, &NewAnchorHandle));
	}
	else
	{
		ensure(TrackedGeometry->GetNativeResource() != nullptr);
		ArTrackable* TrackableHandle = reinterpret_cast<FGoogleARCoreTrackableResource*>(TrackedGeometry->GetNativeResource())->GetNativeHandle();

		ensure(TrackableHandle != nullptr);
		AnchorCreateStatus = ToARCoreAPIStatus(ArTrackable_acquireNewAnchor(SessionHandle, TrackableHandle, PoseHandle, &NewAnchorHandle));
	}
	ArPose_destroy(PoseHandle);

	if (AnchorCreateStatus == EGoogleARCoreAPIStatus::AR_SUCCESS)
	{
		OutAnchor = NewObject<UARPin>();
		OutAnchor->InitARPin(GetARSystem(), ComponentToPin, TransfromInTrackingSpace, TrackedGeometry, InDebugName);
		OutAnchor->SetNativeResource(reinterpret_cast<void*>(NewAnchorHandle));

		UObjectManager->HandleToAnchorMap.Add(FARCorePointer(NewAnchorHandle), OutAnchor);
	}
#endif
	return AnchorCreateStatus;
}

bool FGoogleARCoreSession::TryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InPinName, UARPin*& OutPin)
{
#if PLATFORM_ANDROID
	OutPin = nullptr;
	if (SessionHandle == nullptr ||
		InNativeResource == nullptr)
	{
		return false;
	}

	ArAnchor* AnchorHandle = reinterpret_cast<ArAnchor*>(InNativeResource);

	if (UObjectManager->HandleToAnchorMap.Contains(AnchorHandle))
	{
		OutPin = UObjectManager->HandleToAnchorMap[AnchorHandle];
		return true;
	}

	ArTrackingState AnchorTrackingState;
	ArAnchor_getTrackingState(SessionHandle, AnchorHandle, &AnchorTrackingState);
	if (ToARTrackingState(AnchorTrackingState) != EARTrackingState::Tracking)
	{
		return false;
	}
	
	ArPose* AnchorPoseHandle;
	ArPose_create(SessionHandle, nullptr, &AnchorPoseHandle);
	ArAnchor_getPose(SessionHandle, AnchorHandle, AnchorPoseHandle);
	FTransform UnrealAnchorTransform = ARCorePoseToUnrealTransform(AnchorPoseHandle, SessionHandle, CachedWorldToMeterScale);
	ArPose_destroy(AnchorPoseHandle);

	FName AnchorName{ InPinName };

	OutPin = NewObject<UARPin>();
	OutPin->InitARPin(GetARSystem(), nullptr, UnrealAnchorTransform, nullptr, AnchorName);
	OutPin->SetNativeResource(reinterpret_cast<void*>(AnchorHandle));
	UObjectManager->HandleToAnchorMap.Add(AnchorHandle, OutPin);
	return true;
#else
	OutPin = nullptr;
	return false;
#endif
}

void FGoogleARCoreSession::DetachAnchor(UARPin* Anchor)
{
	auto Key = UObjectManager->HandleToAnchorMap.FindKey(Anchor);
	if (!Key)
	{
		return;
	}

#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return;
	}

	ArAnchor* AnchorHandle = Key->AsRawPointer<ArAnchor>();
	ArAnchor_detach(SessionHandle, AnchorHandle);
	ArAnchor_release(AnchorHandle);

	Anchor->OnTrackingStateChanged(EARTrackingState::StoppedTracking);

	UObjectManager->HandleToAnchorMap.Remove(AnchorHandle);
#endif
}

void FGoogleARCoreSession::GetAllAnchors(TArray<UARPin*>& OutAnchors) const
{
	for (auto Itr : UObjectManager->HandleToAnchorMap)
	{
		OutAnchors.Add(Itr.Value);
	}
}

void FGoogleARCoreSession::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (SessionConfig)
	{
		Collector.AddReferencedObject(SessionConfig);
	}

	if (UObjectManager)
	{
		Collector.AddReferencedObject(UObjectManager);
	}
}

/****************************************/
/*         FGoogleARCoreFrame           */
/****************************************/
FGoogleARCoreFrame::FGoogleARCoreFrame(FGoogleARCoreSession* InSession)
	: Session(InSession)
	, LatestCameraPose(FTransform::Identity)
	, LatestCameraTimestamp(0)
	, LatestCameraTrackingState(EGoogleARCoreTrackingState::StoppedTracking)
	, LatestCameraTrackingFailureReason(EGoogleARCoreTrackingFailureReason::None)
	, LatestPointCloudStatus(EGoogleARCoreAPIStatus::AR_ERROR_SESSION_PAUSED)
	, LatestImageMetadataStatus(EGoogleARCoreAPIStatus::AR_ERROR_SESSION_PAUSED)
{
}

FGoogleARCoreFrame::~FGoogleARCoreFrame()
{
#if PLATFORM_ANDROID
	if (SessionHandle != nullptr)
	{
		ArFrame_destroy(FrameHandle);
		ArPose_destroy(SketchPoseHandle);
	}
#endif
}

void FGoogleARCoreFrame::Init()
{
#if PLATFORM_ANDROID
	if (Session->GetHandle())
	{
		SessionHandle = Session->GetHandle();
		ArFrame_create(SessionHandle, &FrameHandle);
		ArPose_create(SessionHandle, nullptr, &SketchPoseHandle);
	}
#endif
}

bool FGoogleARCoreFrame::UpdateCameraTexture(UARCoreCameraTexture* InCameraTexture) const
{
#if PLATFORM_ANDROID
	SCOPE_CYCLE_COUNTER(STAT_UpdateCameraImage);

	if (!VulkanHardwareBuffer || !InCameraTexture)
	{
		return false;
	}

	InCameraTexture->UpdateCameraImage(VulkanHardwareBuffer);
	return true;
#else
	return false;
#endif // PLATFORM_ANDROID
}

void FGoogleARCoreFrame::UpdateDepthTexture(UARCoreDepthTexture*& OutDepthTexture) const
{
#if PLATFORM_ANDROID
	SCOPE_CYCLE_COUNTER(STAT_UpdateDepthImage);
	ArImage* DepthImage = nullptr;
	if (ArFrame_acquireDepthImage16Bits(SessionHandle, FrameHandle, &DepthImage) == AR_SUCCESS)
	{
		if (!OutDepthTexture)
		{
			OutDepthTexture = UARTexture::CreateARTexture<UARCoreDepthTexture>(EARTextureType::SceneDepthMap);
		}
		OutDepthTexture->UpdateDepthImage(SessionHandle, DepthImage);
		ArImage_release(DepthImage);
	}
#endif
}

void FGoogleARCoreFrame::Update(float WorldToMeterScale)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateARFrame);
	
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return;
	}
	
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateCamera);

		// With Vulkan obtain the Hardware Buffer used by this frame.
		if (FAndroidMisc::ShouldUseVulkan())
		{
			ArFrame_getHardwareBuffer(SessionHandle, FrameHandle, &VulkanHardwareBuffer);
		}
		// With OpenGL obtain the texture Id used by this frame.
		else
		{
			ArFrame_getCameraTextureName(SessionHandle, FrameHandle, &CameraTextureId);
		}

		ArCamera_release(CameraHandle);
		ArFrame_acquireCamera(SessionHandle, FrameHandle, &CameraHandle);

		ArCamera_getDisplayOrientedPose(SessionHandle, CameraHandle, SketchPoseHandle);

		ArTrackingState ARCoreTrackingState;
		ArCamera_getTrackingState(SessionHandle, CameraHandle, &ARCoreTrackingState);
		LatestCameraTrackingState = static_cast<EGoogleARCoreTrackingState>(ARCoreTrackingState);
		LatestCameraPose = ARCorePoseToUnrealTransform(SketchPoseHandle, SessionHandle, WorldToMeterScale);

		LatestCameraTrackingFailureReason = EGoogleARCoreTrackingFailureReason::None;
		ArTrackingFailureReason TrackingFailureReason;
		ArCamera_getTrackingFailureReason(SessionHandle, CameraHandle, &TrackingFailureReason);
		LatestCameraTrackingFailureReason = static_cast<EGoogleARCoreTrackingFailureReason>(TrackingFailureReason);

		int64_t FrameTimestamp = 0;
		ArFrame_getTimestamp(SessionHandle, FrameHandle, &FrameTimestamp);
		LatestCameraTimestamp = FrameTimestamp;
	}
	
	auto ObjectManager = Session->GetUObjectManager();
	auto TrackingSystem = FGoogleARCoreXRTrackingSystem::GetInstance();
	
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdatePointCloud);
		if (LatestCameraTrackingState == EGoogleARCoreTrackingState::Tracking)
		{
			// Update Point Cloud
			UGoogleARCorePointCloud* LatestPointCloud = ObjectManager->LatestPointCloud;
			LatestPointCloud->bIsUpdated = false;
			int64 PreviousTimeStamp = LatestPointCloud->GetUpdateTimestamp();
			ArPointCloud_release(LatestPointCloud->PointCloudHandle);
			LatestPointCloud->PointCloudHandle = nullptr;
			LatestPointCloudStatus = ToARCoreAPIStatus(ArFrame_acquirePointCloud(SessionHandle, FrameHandle, &LatestPointCloud->PointCloudHandle));

			if (PreviousTimeStamp != LatestPointCloud->GetUpdateTimestamp())
			{
				LatestPointCloud->bIsUpdated = true;
			}
		}
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateTrackables);
		// Update trackable that is cached in Unreal
		ArTrackableList* TrackableListHandle = nullptr;
		ArTrackableList_create(SessionHandle, &TrackableListHandle);
		ArSession_getAllTrackables(SessionHandle, AR_TRACKABLE_BASE_TRACKABLE, TrackableListHandle);
		auto Trackables = GetTrackables(SessionHandle, TrackableListHandle, true);
		for (auto TrackableHandle : Trackables)
		{
			// Note that this will create a new UARTrackedGeometry if there's no existing record for it for the handle
			// Note also that there are trackable types we do not support and for which it will return nullptr.
			const FTrackedGeometryGroup* GroupPtr = ObjectManager->GetBaseTrackableFromHandle(TrackableHandle, Session);
			if (GroupPtr)
			{
				const FTrackedGeometryGroup& Group = *GroupPtr;

				// Updated the cached tracked geometry when it is valid.
				FGoogleARCoreTrackableResource* TrackableResource = reinterpret_cast<FGoogleARCoreTrackableResource*>(Group.TrackedGeometry->GetNativeResource());
				TrackableResource->UpdateGeometryData();
			
				//trigger delegates
				if (Group.ARComponent)
				{
					Group.ARComponent->Update(Group.TrackedGeometry);
				}
			
				if (TrackingSystem)
				{
					TrackingSystem->OnTrackableUpdated(Group.TrackedGeometry);
				}
			}
		}
		
		// ARCore doesn't provide a way to mark plane as removed when they're "subsumed"
		// So here we're getting all the trackables in the session and manually remove the ones that are no longer valid
		TArray<UARPin*> ARPinsToRemove;
		ObjectManager->RemoveInvalidTrackables(Trackables, ARPinsToRemove);

		ArTrackableList_destroy(TrackableListHandle);
		
		for (auto Pin : ARPinsToRemove)
		{
			FGoogleARCoreDevice::GetInstance()->RemoveARPin(Pin);
		}
	}
	

	// Update Image Metadata
	ArImageMetadata_release(LatestImageMetadata);
	LatestImageMetadata = nullptr;
	LatestImageMetadataStatus = ToARCoreAPIStatus(ArFrame_acquireImageMetadata(SessionHandle, FrameHandle, &LatestImageMetadata));

	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateAnchors);
		// Update Anchors
		ArAnchorList* UpdatedAnchorListHandle = nullptr;
		ArAnchorList_create(SessionHandle, &UpdatedAnchorListHandle);
		ArFrame_getUpdatedAnchors(SessionHandle, FrameHandle, UpdatedAnchorListHandle);
		int AnchorListSize = 0;
		ArAnchorList_getSize(SessionHandle, UpdatedAnchorListHandle, &AnchorListSize);

		UpdatedAnchors.Empty();
		for (int i = 0; i < AnchorListSize; i++)
		{
			ArAnchor* AnchorHandle = nullptr;
			ArAnchorList_acquireItem(SessionHandle, UpdatedAnchorListHandle, i, &AnchorHandle);

			ArTrackingState AnchorTrackingState;
			ArAnchor_getTrackingState(SessionHandle, AnchorHandle, &AnchorTrackingState);
			if (!ObjectManager->HandleToAnchorMap.Contains(AnchorHandle))
			{
				continue;
			}
			UARPin* AnchorObject = ObjectManager->HandleToAnchorMap[AnchorHandle];
			if (AnchorObject->GetTrackingState() != EARTrackingState::StoppedTracking)
			{
				AnchorObject->OnTrackingStateChanged(ToARTrackingState(AnchorTrackingState));
			}

			if (AnchorObject->GetTrackingState() == EARTrackingState::Tracking)
			{
				ArAnchor_getPose(SessionHandle, AnchorHandle, SketchPoseHandle);
				FTransform AnchorPose = ARCorePoseToUnrealTransform(SketchPoseHandle, SessionHandle, WorldToMeterScale);
				AnchorObject->OnTransformUpdated(AnchorPose);
			}
			UpdatedAnchors.Add(AnchorObject);

			ArAnchor_release(AnchorHandle);
		}
		ArAnchorList_destroy(UpdatedAnchorListHandle);
	}
#endif
}

FTransform FGoogleARCoreFrame::GetCameraPose() const
{
	return LatestCameraPose;
}

int64 FGoogleARCoreFrame::GetCameraTimestamp() const
{
	return LatestCameraTimestamp;
}

EGoogleARCoreTrackingState FGoogleARCoreFrame::GetCameraTrackingState() const
{
	return LatestCameraTrackingState;
}

EGoogleARCoreTrackingFailureReason FGoogleARCoreFrame::GetCameraTrackingFailureReason() const
{
	return LatestCameraTrackingFailureReason;
}

#if PLATFORM_ANDROID
static FARCameraIntrinsics FromCameraIntrinsics(const ArSession* SessionHandle, const ArCameraIntrinsics* NativeCameraIntrinsics)
{
	FARCameraIntrinsics ConvertedIntrinsics;
	FVector2f FocalLength, PrincipalPoint;
	ArCameraIntrinsics_getFocalLength(SessionHandle, NativeCameraIntrinsics, &FocalLength.X, &FocalLength.Y);
	ConvertedIntrinsics.FocalLength = FVector2D(FocalLength);
	ArCameraIntrinsics_getPrincipalPoint(SessionHandle, NativeCameraIntrinsics, &PrincipalPoint.X, &PrincipalPoint.Y);
	ConvertedIntrinsics.PrincipalPoint = FVector2D(PrincipalPoint);
	ArCameraIntrinsics_getImageDimensions(SessionHandle, NativeCameraIntrinsics, &ConvertedIntrinsics.ImageResolution.X, &ConvertedIntrinsics.ImageResolution.Y);
	return ConvertedIntrinsics;
}
#endif

EGoogleARCoreAPIStatus FGoogleARCoreFrame::GetCameraImageIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const
{
	EGoogleARCoreAPIStatus ApiStatus = EGoogleARCoreAPIStatus::AR_SUCCESS;

#if PLATFORM_ANDROID
	ArCameraIntrinsics* NativeCameraIntrinsics = nullptr;
	ArCameraIntrinsics_create(SessionHandle, &NativeCameraIntrinsics);
	ArCamera_getImageIntrinsics(SessionHandle, CameraHandle, NativeCameraIntrinsics);
	OutCameraIntrinsics = FromCameraIntrinsics(SessionHandle, NativeCameraIntrinsics);
	ArCameraIntrinsics_destroy(NativeCameraIntrinsics);
#endif

	return ApiStatus;
}

EGoogleARCoreAPIStatus FGoogleARCoreFrame::GetCameraTextureIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const
{
	EGoogleARCoreAPIStatus ApiStatus = EGoogleARCoreAPIStatus::AR_SUCCESS;
	
#if PLATFORM_ANDROID
	ArCameraIntrinsics* NativeCameraIntrinsics = nullptr;
	ArCameraIntrinsics_create(SessionHandle, &NativeCameraIntrinsics);
	ArCamera_getTextureIntrinsics(SessionHandle, CameraHandle, NativeCameraIntrinsics);
	OutCameraIntrinsics = FromCameraIntrinsics(SessionHandle, NativeCameraIntrinsics);
	ArCameraIntrinsics_destroy(NativeCameraIntrinsics);
#endif
	
	return ApiStatus;
}

void FGoogleARCoreFrame::TransformARCoordinates2D(EGoogleARCoreCoordinates2DType InputCoordinatesType, const TArray<FVector2D>& InputCoordinates, EGoogleARCoreCoordinates2DType OutputCoordinatesType, TArray<FVector2D>& OutputCoordinates) const
{
#if PLATFORM_ANDROID
	ArCoordinates2dType InputType = ToArCoordinates2dType(InputCoordinatesType);
	ArCoordinates2dType OutputType = ToArCoordinates2dType(OutputCoordinatesType);
	int VertNumber = InputCoordinates.Num();

	static TArray<float> InputBuffer;
	static TArray<float> OutputBuffer;

	InputBuffer.Reset(VertNumber * 2);

	for (int i = 0; i < VertNumber; i++)
	{
		InputBuffer.Add(InputCoordinates[i].X);
		InputBuffer.Add(InputCoordinates[i].Y);
	}

	OutputBuffer.SetNumZeroed(VertNumber * 2);
	ArFrame_transformCoordinates2d(SessionHandle, FrameHandle, InputType, VertNumber, InputBuffer.GetData(), OutputType, OutputBuffer.GetData());

	OutputCoordinates.Empty();
	for (int i = 0; i < VertNumber; i++)
	{
		OutputCoordinates.Add(FVector2D(OutputBuffer[i*2], OutputBuffer[i*2 +1]));
	}
#endif
}

void FGoogleARCoreFrame::GetUpdatedAnchors(TArray<UARPin*>& OutUpdatedAnchors) const
{
	OutUpdatedAnchors = UpdatedAnchors;
}

void FGoogleARCoreFrame::ARLineTrace(const FVector2D& ScreenPosition, EGoogleARCoreLineTraceChannel RequestedTraceChannels, TArray<FARTraceResult>& OutHitResults) const
{
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return;
	}

	ArHitResultList *HitResultList = nullptr;
	ArHitResultList_create(SessionHandle, &HitResultList);
	ArFrame_hitTest(SessionHandle, FrameHandle, ScreenPosition.X, ScreenPosition.Y, HitResultList);

	FilterLineTraceResults(HitResultList, RequestedTraceChannels, OutHitResults);

	ArHitResultList_destroy(HitResultList);
#endif
}

void FGoogleARCoreFrame::ARLineTrace(const FVector& Start, const FVector& End, EGoogleARCoreLineTraceChannel RequestedTraceChannels, TArray<FARTraceResult>& OutHitResults) const
{
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return;
	}

	float WorldToMeterScale = Session->GetWorldToMeterScale();
	FVector3f StartInARCore = (FVector3f)UnrealPositionToARCorePosition(Start, WorldToMeterScale);	// LWC_TODO: Precision loss
	FVector3f EndInARCore = (FVector3f)UnrealPositionToARCorePosition(End, WorldToMeterScale);
	FVector3f DirectionInARCore = (EndInARCore - StartInARCore).GetSafeNormal();
	float RayOrigin[3] = { StartInARCore.X, StartInARCore.Y, StartInARCore.Z };
	float RayDirection[3] = { DirectionInARCore.X, DirectionInARCore.Y, DirectionInARCore.Z };

	ArHitResultList *HitResultList = nullptr;
	ArHitResultList_create(SessionHandle, &HitResultList);

	ArFrame_hitTestRay(SessionHandle, FrameHandle, RayOrigin, RayDirection, HitResultList);

	float MaxDistance = FVector::Dist(Start, End);

	FilterLineTraceResults(HitResultList, RequestedTraceChannels, OutHitResults, MaxDistance);

	ArHitResultList_destroy(HitResultList);
#endif
}

bool FGoogleARCoreFrame::IsDisplayRotationChanged() const
{
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return false;
	}

	int Result = 0;
	ArFrame_getDisplayGeometryChanged(SessionHandle, FrameHandle, &Result);
	return Result == 0 ? false : true;
#else
	return false;
#endif
}

FMatrix FGoogleARCoreFrame::GetProjectionMatrix() const
{
	FMatrix44f ProjectionMatrix;

#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return FMatrix();
	}

	ArCamera_getProjectionMatrix(SessionHandle, CameraHandle, GNearClippingPlane, 100.0f, ProjectionMatrix.M[0]);

	
	// We need to multiple the center offset by -1 to get the correct projection matrix in Unreal.
	ProjectionMatrix.M[2][0] = -1.0f * ProjectionMatrix.M[2][0];
	ProjectionMatrix.M[2][1] = -1.0f * ProjectionMatrix.M[2][1];

	// Unreal uses the infinite far plane project matrix.
	ProjectionMatrix.M[2][2] = 0.0f;
	ProjectionMatrix.M[2][3] = 1.0f;
	ProjectionMatrix.M[3][2] = GNearClippingPlane;
#endif
	return FMatrix(ProjectionMatrix);
}

void FGoogleARCoreFrame::TransformDisplayUvCoords(const TArray<float>& UvCoords, TArray<float>& OutUvCoords) const
{
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return;
	}

	OutUvCoords.SetNumZeroed(8);
	ArFrame_transformCoordinates2d(SessionHandle, FrameHandle, AR_COORDINATES_2D_VIEW_NORMALIZED, 4, UvCoords.GetData(), AR_COORDINATES_2D_TEXTURE_NORMALIZED, OutUvCoords.GetData());
#endif
}

FGoogleARCoreLightEstimate FGoogleARCoreFrame::GetLightEstimate() const
{
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return FGoogleARCoreLightEstimate();
	}

	ArLightEstimate *LightEstimateHandle = nullptr;
	ArLightEstimate_create(SessionHandle, &LightEstimateHandle);
	ArFrame_getLightEstimate(SessionHandle, FrameHandle, LightEstimateHandle);

	ArLightEstimateState LightEstimateState;
	ArLightEstimate_getState(SessionHandle, LightEstimateHandle, &LightEstimateState);

	FGoogleARCoreLightEstimate LightEstimate;
	LightEstimate.bIsValid = (LightEstimateState == AR_LIGHT_ESTIMATE_STATE_VALID) ? true : false;

	if(LightEstimate.bIsValid)
	{
		ArLightEstimate_getPixelIntensity(SessionHandle, LightEstimateHandle, &LightEstimate.PixelIntensity);

		float ColorCorrectionVector[4] ;
		ArLightEstimate_getColorCorrection(SessionHandle, LightEstimateHandle, ColorCorrectionVector);

		LightEstimate.RGBScaleFactor = FVector(ColorCorrectionVector[0], ColorCorrectionVector[1], ColorCorrectionVector[2]);
		LightEstimate.PixelIntensity = ColorCorrectionVector[3];
	}
	else
	{
		LightEstimate.RGBScaleFactor = FVector(0.0f, 0.0f, 0.0f);
		LightEstimate.PixelIntensity = 0.0f;
	}

	ArLightEstimate_destroy(LightEstimateHandle);

	return LightEstimate;
#else
	return FGoogleARCoreLightEstimate();
#endif
}

EGoogleARCoreAPIStatus FGoogleARCoreFrame::GetPointCloud(UGoogleARCorePointCloud*& OutLatestPointCloud) const
{
	OutLatestPointCloud = Session->GetUObjectManager()->LatestPointCloud;
	return LatestPointCloudStatus;
}

EGoogleARCoreAPIStatus FGoogleARCoreFrame::AcquirePointCloud(UGoogleARCorePointCloud*& OutLatestPointCloud) const
{
	OutLatestPointCloud = nullptr;
	EGoogleARCoreAPIStatus AcquirePointCloudStatus = EGoogleARCoreAPIStatus::AR_SUCCESS;
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return EGoogleARCoreAPIStatus::AR_ERROR_SESSION_PAUSED;
	}

	ArPointCloud* PointCloudHandle = nullptr;
	AcquirePointCloudStatus = ToARCoreAPIStatus(ArFrame_acquirePointCloud(SessionHandle, FrameHandle, &PointCloudHandle));

	if (AcquirePointCloudStatus == EGoogleARCoreAPIStatus::AR_SUCCESS)
	{
		OutLatestPointCloud = NewObject<UGoogleARCorePointCloud>();
		OutLatestPointCloud->Session = Session->AsShared();
		OutLatestPointCloud->PointCloudHandle = PointCloudHandle;
		OutLatestPointCloud->bIsUpdated = true;
	}
	else
	{
		UE_LOGF(LogGoogleARCoreAPI, Error, "AcquirePointCloud failed due to resource exhausted!");
	}
#endif
	return AcquirePointCloudStatus;
}

EGoogleARCoreAPIStatus FGoogleARCoreFrame::AcquireCameraImage(UGoogleARCoreCameraImage *&OutCameraImage) const
{
	EGoogleARCoreAPIStatus ApiStatus = EGoogleARCoreAPIStatus::AR_SUCCESS;
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return EGoogleARCoreAPIStatus::AR_ERROR_SESSION_PAUSED;
	}

	ArImage *OutImage = nullptr;
	ApiStatus = ToARCoreAPIStatus(
		ArFrame_acquireCameraImage(
			const_cast<ArSession*>(SessionHandle), FrameHandle, &OutImage));

	if (ApiStatus == EGoogleARCoreAPIStatus::AR_SUCCESS)
	{
		OutCameraImage = NewObject<UGoogleARCoreCameraImage>();
		OutCameraImage->ArImage = OutImage;
		OutCameraImage->SessionHandle = SessionHandle;
	}
	else
	{
		UE_LOGF(LogGoogleARCoreAPI, Error, "AcquireCameraImage failed!");
	}
#endif

	return ApiStatus;
}

TSharedPtr<FGoogleARCoreSession> FGoogleARCoreSession::CreateARCoreSession()
{
	TSharedPtr<FGoogleARCoreSession> NewSession = MakeShared<FGoogleARCoreSession>();

	UGoogleARCoreUObjectManager* UObjectManager = NewObject<UGoogleARCoreUObjectManager>();
	UObjectManager->LatestPointCloud = NewObject<UGoogleARCorePointCloud>();
	UObjectManager->LatestPointCloud->Session = NewSession;
	
	NewSession->UObjectManager = UObjectManager;
	return NewSession;
}

#if PLATFORM_ANDROID
void FGoogleARCoreFrame::FilterLineTraceResults(ArHitResultList* HitResultList, EGoogleARCoreLineTraceChannel RequestedTraceChannels, TArray<FARTraceResult>& OutHitResults, float MaxDistance) const
{
	ArHitResult* HitResultHandle = nullptr;
	ArPose* HitResultPoseHandle = nullptr;
	int32_t HitResultCount = 0;

	ArPose_create(SessionHandle, nullptr, &HitResultPoseHandle);
	ArHitResultList_getSize(SessionHandle, HitResultList, &HitResultCount);
	ArHitResult_create(SessionHandle, &HitResultHandle);
	for (int32_t i = 0; i < HitResultCount; i++)
	{
		ArHitResultList_getItem(SessionHandle, HitResultList, i, HitResultHandle);

		float Distance = 0;
		ArHitResult_getDistance(SessionHandle, HitResultHandle, &Distance);
		Distance *= Session->GetWorldToMeterScale();

		ArHitResult_getHitPose(SessionHandle, HitResultHandle, HitResultPoseHandle);
		FTransform HitTransform = ARCorePoseToUnrealTransform(HitResultPoseHandle, SessionHandle, Session->GetWorldToMeterScale());

		ArTrackable* TrackableHandle = nullptr;
		ArHitResult_acquireTrackable(SessionHandle, HitResultHandle, &TrackableHandle);

		ensure(TrackableHandle != nullptr);

		ArTrackableType TrackableType = ArTrackableType::AR_TRACKABLE_NOT_VALID;
		ArTrackable_getType(SessionHandle, TrackableHandle, &TrackableType);

		// Filter the HitResult based on the requested trace channel.
		if (TrackableType == AR_TRACKABLE_POINT)
		{
			ArPoint* ARPointHandle = reinterpret_cast<ArPoint*>(TrackableHandle);
			ArPointOrientationMode OrientationMode = AR_POINT_ORIENTATION_INITIALIZED_TO_IDENTITY;
			ArPoint_getOrientationMode(SessionHandle, ARPointHandle, &OrientationMode);
			if (OrientationMode == AR_POINT_ORIENTATION_ESTIMATED_SURFACE_NORMAL && !!(RequestedTraceChannels & EGoogleARCoreLineTraceChannel::FeaturePointWithSurfaceNormal))
			{
				UARTrackedGeometry* TrackedGeometry = Session->GetUObjectManager()->GetTrackableFromHandle<UARTrackedGeometry>(TrackableHandle, Session);
				check(TrackedGeometry);
				FARTraceResult UEHitResult(Session->GetARSystem(), Distance, EARLineTraceChannels::FeaturePoint, HitTransform, TrackedGeometry);
				OutHitResults.Add(UEHitResult);
				continue;
			}
			if (!!(RequestedTraceChannels & EGoogleARCoreLineTraceChannel::FeaturePoint))
			{
				UARTrackedGeometry* TrackedGeometry = Session->GetUObjectManager()->GetTrackableFromHandle<UARTrackedGeometry>(TrackableHandle, Session);
				check(TrackedGeometry);
				FARTraceResult UEHitResult(Session->GetARSystem(), Distance, EARLineTraceChannels::FeaturePoint, HitTransform, TrackedGeometry);
				OutHitResults.Add(UEHitResult);
				continue;
			}
		}
		if (TrackableType == AR_TRACKABLE_PLANE)
		{
			ArPlane* PlaneHandle = reinterpret_cast<ArPlane*>(TrackableHandle);
			if (!!(RequestedTraceChannels & EGoogleARCoreLineTraceChannel::PlaneUsingBoundaryPolygon))
			{
				int32 PointInsidePolygon = 0;
				ArPlane_isPoseInPolygon(SessionHandle, PlaneHandle, HitResultPoseHandle, &PointInsidePolygon);
				if (PointInsidePolygon)
				{
					UARTrackedGeometry* TrackedGeometry = Session->GetUObjectManager()->GetTrackableFromHandle<UARTrackedGeometry>(TrackableHandle, Session);
					check(TrackedGeometry);
					FARTraceResult UEHitResult(Session->GetARSystem(), Distance, EARLineTraceChannels::PlaneUsingBoundaryPolygon, HitTransform, TrackedGeometry);
					OutHitResults.Add(UEHitResult);
					continue;
				}
			}
			if (!!(RequestedTraceChannels & EGoogleARCoreLineTraceChannel::PlaneUsingExtent))
			{
				int32 PointInsideExtents = 0;
				ArPlane_isPoseInExtents(SessionHandle, PlaneHandle, HitResultPoseHandle, &PointInsideExtents);
				if (PointInsideExtents)
				{
					UARTrackedGeometry* TrackedGeometry = Session->GetUObjectManager()->GetTrackableFromHandle<UARTrackedGeometry>(TrackableHandle, Session);
					check(TrackedGeometry);
					FARTraceResult UEHitResult(Session->GetARSystem(), Distance, EARLineTraceChannels::PlaneUsingExtent, HitTransform, TrackedGeometry);
					OutHitResults.Add(UEHitResult);
					continue;
				}
			}
			if (!!(RequestedTraceChannels & EGoogleARCoreLineTraceChannel::InfinitePlane))
			{
				UARTrackedGeometry* TrackedGeometry = Session->GetUObjectManager()->GetTrackableFromHandle<UARTrackedGeometry>(TrackableHandle, Session);
				check(TrackedGeometry);
				FARTraceResult UEHitResult(Session->GetARSystem(), Distance, EARLineTraceChannels::GroundPlane, HitTransform, TrackedGeometry);
				OutHitResults.Add(UEHitResult);
				continue;
			}
		}
		if (TrackableType == AR_TRACKABLE_AUGMENTED_IMAGE)
		{
			if (!!(RequestedTraceChannels & EGoogleARCoreLineTraceChannel::AugmentedImage))
			{
				UARTrackedGeometry* TrackedGeometry = Session->GetUObjectManager()->GetTrackableFromHandle<UARTrackedGeometry>(TrackableHandle, Session);
				check(TrackedGeometry);
				FARTraceResult UEHitResult(Session->GetARSystem(), Distance, EARLineTraceChannels::PlaneUsingExtent, HitTransform, TrackedGeometry);
				OutHitResults.Add(UEHitResult);
				continue;
			}
		}

		// If we don't pass the handle into GetTrackableFromHandle we need to release it here.
		ArTrackable_release(TrackableHandle);
	}

	ArHitResult_destroy(HitResultHandle);
	ArPose_destroy(HitResultPoseHandle);
}
#endif

/****************************************/
/*       UGoogleARCorePointCloud        */
/****************************************/
UGoogleARCorePointCloud::~UGoogleARCorePointCloud()
{
	ReleasePointCloud();
}

int64 UGoogleARCorePointCloud::GetUpdateTimestamp()
{
	if (CheckIsSessionValid("ARCorePointCloud", Session))
	{
#if PLATFORM_ANDROID
		int64_t TimeStamp = 0;
		ArPointCloud_getTimestamp(Session.Pin()->GetHandle(), PointCloudHandle, &TimeStamp);
		return TimeStamp;
#endif
	}
	return 0;
}

bool UGoogleARCorePointCloud::IsUpdated()
{
	return bIsUpdated;
}

int UGoogleARCorePointCloud::GetPointNum()
{
	int PointNum = 0;
	if (CheckIsSessionValid("ARCorePointCloud", Session))
	{
#if PLATFORM_ANDROID
		ArPointCloud_getNumberOfPoints(Session.Pin()->GetHandle(), PointCloudHandle, &PointNum);
#endif
	}
	return PointNum;
}

void UGoogleARCorePointCloud::GetPoint(int Index, FVector& OutWorldPosition, float& OutConfidence)
{
	FVector Point = FVector::ZeroVector;
	float Confidence = 0.0;
	if (CheckIsSessionValid("ARCorePointCloud", Session))
	{
#if PLATFORM_ANDROID
		const float* PointData = nullptr;
		ArPointCloud_getData(Session.Pin()->GetHandle(), PointCloudHandle, &PointData);

		Point.Y = PointData[Index * 4];
		Point.Z = PointData[Index * 4 + 1];
		Point.X = -PointData[Index * 4 + 2];

		Point = Point * Session.Pin()->GetWorldToMeterScale();
		FTransform PointLocalTransform(Point);
		TSharedRef<FARSupportInterface , ESPMode::ThreadSafe> ARSystem = Session.Pin()->GetARSystem();
		FTransform PointWorldTransform = PointLocalTransform * ARSystem->GetAlignmentTransform() * ARSystem->GetXRTrackingSystem()->GetTrackingToWorldTransform();
		Point = PointWorldTransform.GetTranslation();
		Confidence = PointData[Index * 4 + 3];
#endif
	}
	OutWorldPosition = Point;
	OutConfidence = Confidence;
}

int UGoogleARCorePointCloud::GetPointId(int Index)
{
	int Id = 0;
	if (CheckIsSessionValid("ARCorePointCloud", Session))
	{
#if PLATFORM_ANDROID
		const int32_t* Ids = 0;
		ArPointCloud_getPointIds(Session.Pin()->GetHandle(), PointCloudHandle, &Ids);
		Id = Ids[Index];
#endif
	}
	return Id;
}

void UGoogleARCorePointCloud::GetPointInTrackingSpace(int Index, FVector& OutTrackingSpaceLocation, float& OutConfidence)
{
	FVector Point = FVector::ZeroVector;
	float Confidence = 0.0;
	if (CheckIsSessionValid("ARCorePointCloud", Session))
	{
#if PLATFORM_ANDROID
		const float* PointData = nullptr;
		ArPointCloud_getData(Session.Pin()->GetHandle(), PointCloudHandle, &PointData);

		Point.Y = PointData[Index * 4];
		Point.Z = PointData[Index * 4 + 1];
		Point.X = -PointData[Index * 4 + 2];
		Confidence = PointData[Index * 4 + 3];

		Point = Point * Session.Pin()->GetWorldToMeterScale();
#endif
	}
	OutTrackingSpaceLocation = Point;
	OutConfidence = Confidence;
}

void UGoogleARCorePointCloud::ReleasePointCloud()
{
#if PLATFORM_ANDROID
	if (PointCloudHandle)
	{
		ArPointCloud_release(PointCloudHandle);
		PointCloudHandle = nullptr;
	}
#endif
}

#if PLATFORM_ANDROID
void UGoogleARCoreUObjectManager::DumpTrackableHandleMap(const ArSession* SessionHandle)
{
	for (auto KeyValuePair : TrackableHandleMap)
	{
		ArTrackable* TrackableHandle = KeyValuePair.Key.AsRawPointer<ArTrackable>();
		UARTrackedGeometry* TrackedGeometry = KeyValuePair.Value.TrackedGeometry;

		ArTrackableType TrackableType = ArTrackableType::AR_TRACKABLE_NOT_VALID;
		ArTrackable_getType(SessionHandle, TrackableHandle, &TrackableType);
		ArTrackingState ARTrackingState = ArTrackingState::AR_TRACKING_STATE_STOPPED;
		ArTrackable_getTrackingState(SessionHandle, TrackableHandle, &ARTrackingState);

		UE_LOGF(LogGoogleARCoreAPI, Log, "TrackableHandle - address: %p, type: 0x%x, tracking state: %d",
			TrackableHandle, (int)TrackableType, (int)ARTrackingState);
		if (TrackedGeometry != nullptr)
		{
			FGoogleARCoreTrackableResource* NativeResource = reinterpret_cast<FGoogleARCoreTrackableResource*>(TrackedGeometry->GetNativeResource());
			UE_LOGF(LogGoogleARCoreAPI, Log, "TrackedGeometry - NativeResource:%p, type: %ls, tracking state: %d",
				NativeResource->GetNativeHandle(), *TrackedGeometry->GetClass()->GetFName().ToString(), (int)TrackedGeometry->GetTrackingState());
		}
		else
		{
			UE_LOGF(LogGoogleARCoreAPI, Log, "TrackedGeometry - InValid or Pending Kill.")
		}
	}
}


// Wraps a pointer inside a guid by xoring them with the last 64 bits of a constant base guid
FGuid UGoogleARCoreUObjectManager::TrackableHandleToGuid(const ArTrackable* TrackableHandle)
{

	FGuid Guid = BaseGuid;
	if (TrackableHandle == nullptr)
	{
		Guid.Invalidate();
	}
	else
	{
		UPTRINT IntPtr = reinterpret_cast<UPTRINT>(TrackableHandle);
		if (sizeof(UPTRINT) > 4)
		{
			Guid[0] ^= (static_cast<uint64>(IntPtr) >> 32);
		}
		Guid[1] ^= IntPtr & 0xFFFFFFFF;
	}
	return Guid;
}

// Unwrap a pointer previously wrapped using TrackabeHandleToGuid. The caller must somehow validate the returned pointer, 
// for instance by checking whether it is in TrackableHandleMap
ArTrackable* UGoogleARCoreUObjectManager::GuidToTrackableHandle(const FGuid& Guid)
{
	UPTRINT IntPtr = 0;

	// Check whether the unmodified parts of the guid match the base guid belonging to this instance
	if (Guid[2] != BaseGuid[2] || Guid[3] != BaseGuid[3] || (sizeof(UPTRINT) == 4 && Guid[0] != BaseGuid[0]))
	{
		return nullptr;
	}
	if (sizeof(UPTRINT) > 4)
	{
		IntPtr = static_cast<UPTRINT>(static_cast<uint64>(Guid[0] ^ BaseGuid[0]) << 32);
	}
	IntPtr |= (Guid[1] ^ BaseGuid[1]) & 0xFFFFFFFF;

	ArTrackable* Result = reinterpret_cast<ArTrackable*>(IntPtr);
	//check(TrackableHandleMap.Contains(Result));
	return Result;
}

#endif

void* FGoogleARCoreSession::GetLatestFrameRawPointer()
{
#if PLATFORM_ANDROID
	return reinterpret_cast<void*>(LatestFrame->GetHandle());
#else
	return nullptr;
#endif
}

bool FGoogleARCoreSession::IsSceneDepthEnabled() const
{
#if PLATFORM_ANDROID
	if (SessionHandle && ConfigHandle)
	{
		ArDepthMode DepthMode = AR_DEPTH_MODE_DISABLED;
		ArConfig_getDepthMode(SessionHandle, ConfigHandle, &DepthMode);
		return DepthMode != AR_DEPTH_MODE_DISABLED;
	}
#endif
	
	return false;
}

UGoogleARCoreUObjectManager::UGoogleARCoreUObjectManager()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		SpawnARActorDelegateHandle = UARLifeCycleComponent::OnSpawnARActorDelegate.AddUObject(this, &UGoogleARCoreUObjectManager::OnSpawnARActor);
#if PLATFORM_ANDROID
		BaseGuid = FGuid::NewGuid();
#endif
	}
}

UGoogleARCoreUObjectManager::~UGoogleARCoreUObjectManager()
{
	UARLifeCycleComponent::OnSpawnARActorDelegate.Remove(SpawnARActorDelegateHandle);
}


UARSessionConfig& UGoogleARCoreUObjectManager::AccessSessionConfig()
{
	return FGoogleARCoreDevice::GetInstance()->GetARSystem()->AccessSessionConfig();
}

void UGoogleARCoreUObjectManager::OnSpawnARActor(AARActor* NewARActor, UARComponent* NewARComponent, FGuid NativeID)
{
#if PLATFORM_ANDROID
	FTrackedGeometryGroup* TrackedGeometryGroup = TrackableHandleMap.Find(GuidToTrackableHandle(NativeID));
	if (TrackedGeometryGroup != nullptr)
	{
		//this should still be null
		check(TrackedGeometryGroup->ARActor == nullptr);
		check(TrackedGeometryGroup->ARComponent == nullptr);

		check(NewARActor);
		check(NewARComponent);

		TrackedGeometryGroup->ARActor = NewARActor;
		TrackedGeometryGroup->ARComponent = NewARComponent;
		
		//NOW, we can make the callbacks
		TrackedGeometryGroup->ARComponent->Update(TrackedGeometryGroup->TrackedGeometry);
		if (auto TrackingSystem = FGoogleARCoreXRTrackingSystem::GetInstance())
		{
			TrackingSystem->OnTrackableAdded(TrackedGeometryGroup->TrackedGeometry);
		}
	}
	else
	{
		UE_LOGF(LogGoogleARCoreAPI, Warning, "AR NativeID not found.  Make sure to set this on the ARComponent!");
	}
#endif
}

FGoogleARCoreCameraConfig::FGoogleARCoreCameraConfig()
{
#if PLATFORM_ANDROID
	static_assert((int32)EGoogleARCoreCameraFPS::FPS_30 == AR_CAMERA_CONFIG_TARGET_FPS_30, "");
	static_assert((int32)EGoogleARCoreCameraFPS::FPS_60 == AR_CAMERA_CONFIG_TARGET_FPS_60, "");
	static_assert((int32)EGoogleARCoreCameraDepthSensorUsage::DepthSensor_RequireAndUse == AR_CAMERA_CONFIG_DEPTH_SENSOR_USAGE_REQUIRE_AND_USE, "");
	static_assert((int32)EGoogleARCoreCameraDepthSensorUsage::DepthSensor_DoNotUse == AR_CAMERA_CONFIG_DEPTH_SENSOR_USAGE_DO_NOT_USE, "");
#endif
}

int32 FGoogleARCoreCameraConfig::GetMaxFPS() const
{
	return TargetFPS & (int32)EGoogleARCoreCameraFPS::FPS_60 ? 60 : 30;
}

void FGoogleARCoreCameraConfig::SetMaxFPS(int32 MaxFPS)
{
	if (MaxFPS > 30)
	{
		TargetFPS = (int32)EGoogleARCoreCameraFPS::FPS_60 | (int32)EGoogleARCoreCameraFPS::FPS_30;
	}
	else
	{
		TargetFPS = (int32)EGoogleARCoreCameraFPS::FPS_30;
	}
}

FString FGoogleARCoreCameraConfig::ToLogString() const
{
	FString LogString = FString::Printf(TEXT("CameraImageResolution (%d x %d), CameraTextureResolution (%d x %d), CameraID (%s), TargetFPS Mode (%d, %d Max FPS), DepthSensorUsage Mode (%d), Facing (%i)"),
		CameraImageResolution.X, CameraImageResolution.Y, CameraTextureResolution.X, CameraTextureResolution.Y, *CameraID, TargetFPS, GetMaxFPS(), DepthSensorUsage, (int32)CameraFacing);
	return MoveTemp(LogString);
}

bool FGoogleARCoreCameraConfig::IsCompatibleWith(const FGoogleARCoreCameraConfig& OtherConfig) const
{
	if (CameraImageResolution != OtherConfig.CameraImageResolution ||
		CameraTextureResolution != OtherConfig.CameraTextureResolution ||
		CameraID != OtherConfig.CameraID ||
		CameraFacing != OtherConfig.CameraFacing)
	{
		return false;
	}

	static const int32 FPS_ALL = (int32)EGoogleARCoreCameraFPS::FPS_30 | (int32)EGoogleARCoreCameraFPS::FPS_60;
	static const int32 DepthSensorUsage_ALL = (int32)EGoogleARCoreCameraDepthSensorUsage::DepthSensor_RequireAndUse | (int32)EGoogleARCoreCameraDepthSensorUsage::DepthSensor_DoNotUse;

	/**
		Not specifying the target FPS or the depth sensor usage means none of the filtering is applied, which equals to selecting all the filtering options.
		And for 2 configs to be considered compatible, their filtering options only need to overlap - for instance, requesting both 30 and 60 FPS, but only get 30 FPS back
		To make it easier for the overlap check, the "none filtering option" case needs to be converted to the "all filtering options" version
	*/
	int32 MyTargetFPS = TargetFPS ? TargetFPS : FPS_ALL;
	int32 OtherTargetFPS = OtherConfig.TargetFPS ? OtherConfig.TargetFPS : FPS_ALL;

	int32 MyDepthSensorUsage = DepthSensorUsage ? DepthSensorUsage : DepthSensorUsage_ALL;
	int32 OtherDepthSensorUsage = OtherConfig.DepthSensorUsage ? OtherConfig.DepthSensorUsage : DepthSensorUsage_ALL;

	return (MyTargetFPS & OtherTargetFPS) && (MyDepthSensorUsage & OtherDepthSensorUsage);
}

UARTrackedGeometry* UGoogleARCoreUObjectManager::RemoveTrackable(FARCorePointer Pointer)
{
	if (auto Record = TrackableHandleMap.Find(Pointer))
	{
		if (Record->ARComponent)
		{
			Record->ARComponent->Remove(Record->TrackedGeometry);
			AARActor::RequestDestroyARActor(Record->ARActor);
		}
		
		if (Record->TrackedGeometry)
		{
			Record->TrackedGeometry->UpdateTrackingState(EARTrackingState::StoppedTracking);
			if (auto TrackingSystem = FGoogleARCoreXRTrackingSystem::GetInstance())
			{
				TrackingSystem->OnTrackableRemoved(Record->TrackedGeometry);
			}
		}
		
		auto RemovedGeometry = Record->TrackedGeometry;
		TrackableHandleMap.Remove(Pointer);
		return RemovedGeometry;
	}
	
	return nullptr;
}

void UGoogleARCoreUObjectManager::ClearTrackables()
{
	TArray<FARCorePointer> Pointers;
	TrackableHandleMap.GetKeys(Pointers);
	for (auto Pointer : Pointers)
	{
		RemoveTrackable(Pointer);
	}
	
	// The stored UARTrackedGeometry* will be deleted by GC
	TrackableHandleMap = {};
}

#if PLATFORM_ANDROID
const FTrackedGeometryGroup* UGoogleARCoreUObjectManager::GetBaseTrackableFromHandle(ArTrackable* TrackableHandle, FGoogleARCoreSession* Session)
{
	if (!TrackableHandleMap.Contains(TrackableHandle)
		|| (TrackableHandleMap[TrackableHandle].TrackedGeometry == nullptr)
		|| TrackableHandleMap[TrackableHandle].TrackedGeometry->GetTrackingState() == EARTrackingState::StoppedTracking)
	{
		// Add the trackable to the cache.
		UARTrackedGeometry* NewTrackableObject = nullptr;
		ArTrackableType TrackableType = ArTrackableType::AR_TRACKABLE_NOT_VALID;
		ArTrackable_getType(Session->GetHandle(), TrackableHandle, &TrackableType);
		IARRef* NativeResource = nullptr;
		UClass* ARComponentClass = nullptr;
		if (TrackableType == ArTrackableType::AR_TRACKABLE_PLANE)
		{
			NewTrackableObject = NewObject<UARPlaneGeometry>();
			NativeResource = new FGoogleARCoreTrackedPlaneResource(Session->AsShared(), TrackableHandle, NewTrackableObject);
			ARComponentClass = AccessSessionConfig().GetPlaneComponentClass();
		}
		else if (TrackableType == ArTrackableType::AR_TRACKABLE_POINT)
		{
			NewTrackableObject = NewObject<UARTrackedPoint>();
			NativeResource = new FGoogleARCoreTrackedPointResource(Session->AsShared(), TrackableHandle, NewTrackableObject);
			ARComponentClass = AccessSessionConfig().GetPointComponentClass();
		}
		else if (TrackableType == ArTrackableType::AR_TRACKABLE_AUGMENTED_IMAGE)
		{
			NewTrackableObject = NewObject<UGoogleARCoreAugmentedImage>();
			NativeResource = new FGoogleARCoreAugmentedImageResource(Session->AsShared(), TrackableHandle, NewTrackableObject);
			ARComponentClass = AccessSessionConfig().GetImageComponentClass();
		}
		else if (TrackableType == ArTrackableType::AR_TRACKABLE_FACE)
		{
			NewTrackableObject = NewObject<UGoogleARCoreAugmentedFace>();
			NativeResource = new FGoogleARCoreAugmentedFaceResource(Session->AsShared(), TrackableHandle, NewTrackableObject);
			ARComponentClass = AccessSessionConfig().GetFaceComponentClass();
		}
		else if (TrackableType == ArTrackableType::AR_TRACKABLE_DEPTH_POINT)
		{
			// If depth sensing is enabled to get the depth texture ar ray casts will create 'depth point' trackables.
			// However depth point trackables position can only be accessed by creating an ArAnchor from them.  This breaks a lot of assumptions in the 
			// ARTrackedGeometry system design.  Unless there is some refactoring this one trackable will work quite differently from the others.
			// For now we will just ignore these depth point trackables.
			ArTrackable_release(TrackableHandle);
			return nullptr;
		}
		// Currently we do not enable this trackable type with ArConfig_setInstantPlacementMode, so we should never have one.
		check(TrackableType != ArTrackableType::AR_TRACKABLE_INSTANT_PLACEMENT_POINT);
		
		// We should have a valid trackable object now.
		checkf(NewTrackableObject, TEXT("Unknown ARCore Trackable Type: %d"), TrackableType);

		NewTrackableObject->InitializeNativeResource(NativeResource);
		NativeResource = nullptr;

		FGoogleARCoreTrackableResource* TrackableResource = reinterpret_cast<FGoogleARCoreTrackableResource*>(NewTrackableObject->GetNativeResource());
		
		FTrackedGeometryGroup TrackedGeometryGroup(NewTrackableObject);
		
		// Update the tracked geometry data using the native resource
		TrackableResource->UpdateGeometryData();
		
		// Add the new object to the record
		TrackableHandleMap.Add(TrackableHandle, TrackedGeometryGroup);
		
		UE_LOGF(LogGoogleARCoreAPI, Log, "Added NewTrackableObject: %ls %ls for handle 0x%llx of type %i, current total number of trackables: %d",
			*NewTrackableObject->GetName(), *NewTrackableObject->GetDebugName().ToString(), TrackableHandle, (int32)TrackableType, TrackableHandleMap.Num());
		
		const auto Guid = TrackableHandleToGuid(TrackableHandle);
		NewTrackableObject->UniqueId = Guid;
		AARActor::RequestSpawnARActor(Guid, ARComponentClass);
	}
	else
	{
		// If we are not create new trackable object, release the trackable handle.
		ArTrackable_release(TrackableHandle);
	}
	
	const auto& Group = TrackableHandleMap[TrackableHandle];
	checkf(Group.TrackedGeometry, TEXT("UGoogleARCoreUObjectManager failed to get a valid trackable %p from the map."), TrackableHandle);
	return &Group;
}

void UGoogleARCoreUObjectManager::RemoveInvalidTrackables(const TArray<ArTrackable*>& ValidTrackables, TArray<UARPin*>& ARPinsToRemove)
{
	TArray<FARCorePointer> InvalidPointers;
	for (auto Itr : TrackableHandleMap)
	{
		auto TrackableHandle = Itr.Key.AsRawPointer<ArTrackable>();
		
		if (!ValidTrackables.Contains(TrackableHandle))
		{
			InvalidPointers.Add(Itr.Key);
		}
	}
	
	TArray<UARTrackedGeometry*> RemovedGeometries;
	for (auto Pointer : InvalidPointers)
	{
		if (auto RemovedGeometry = RemoveTrackable(Pointer))
		{
			RemovedGeometries.Add(RemovedGeometry);
		}
	}
	
	for (auto Itr : HandleToAnchorMap)
	{
		if (auto Pin = Itr.Value)
		{
			if (RemovedGeometries.Contains(Pin->GetTrackedGeometry()))
			{
				ARPinsToRemove.Add(Pin);
			}
		}
	}
}

TArray<ArTrackable*> FGoogleARCoreFrame::GetTrackables(const ArSession* SessionHandle, ArTrackableList* ListHandle, bool bRemoveSubsumedPlanes)
{
	TArray<ArTrackable*> Trackables;
	if (SessionHandle && ListHandle)
	{
		int TrackableListSize = 0;
		ArTrackableList_getSize(SessionHandle, ListHandle, &TrackableListSize);
		for (auto Index = 0; Index < TrackableListSize; ++Index)
		{
			ArTrackable* TrackableHandle = nullptr;
			ArTrackableList_acquireItem(SessionHandle, ListHandle, Index, &TrackableHandle);
			
			// Check whether this is a plane that's subsumed
			auto bSubsumed = false;
			if (bRemoveSubsumedPlanes)
			{
				ArTrackableType TrackableType;
				ArTrackable_getType(SessionHandle, TrackableHandle, &TrackableType);
				if (TrackableType == AR_TRACKABLE_PLANE)
				{
					ArPlane* SubsumedByPlaneHandle = nullptr;
					ArPlane_acquireSubsumedBy(SessionHandle, (ArPlane*)TrackableHandle, &SubsumedByPlaneHandle);
					if (SubsumedByPlaneHandle)
					{
						bSubsumed = true;
					}
				}
			}
			else
			{
				bSubsumed = false;
			}
			
			if (bSubsumed)
			{
				ArTrackable_release(TrackableHandle);
			}
			else
			{
				Trackables.Add(TrackableHandle);
			}
		}
	}
	
	return MoveTemp(Trackables);
}
#endif
