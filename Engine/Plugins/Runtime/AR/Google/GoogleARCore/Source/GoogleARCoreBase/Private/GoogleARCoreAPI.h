// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "GoogleARCoreTypes.h"
#include "GoogleARCoreSessionConfig.h"
#include "GoogleARCoreAugmentedImage.h"
#include "GoogleARCoreAugmentedFace.h"
#include "GoogleARCoreCameraIntrinsics.h"
#include "GoogleARCoreAugmentedImageDatabase.h"
#include "ARActor.h"
#include "ARSessionConfig.h"

#if PLATFORM_ANDROID
#include "arcore_c_api.h"
#endif

#include "GoogleARCoreAPI.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGoogleARCoreAPI, Log, All);
class UARCoreCameraTexture;
class UARCoreDepthTexture;

enum class EGoogleARCoreAPIStatus : int
{
	/// The operation was successful.
	AR_SUCCESS = 0,

	/// One of the arguments was invalid, either null or not appropriate for the
	/// operation requested.
	AR_ERROR_INVALID_ARGUMENT = -1,

	/// An internal error occurred that the application should not attempt to
	/// recover from.
	AR_ERROR_FATAL = -2,

	/// An operation was attempted that requires the session be running, but the
	/// session was paused.
	AR_ERROR_SESSION_PAUSED = -3,

	/// An operation was attempted that requires the session be paused, but the
	/// session was running.
	AR_ERROR_SESSION_NOT_PAUSED = -4,

	/// An operation was attempted that the session be in the TRACKING state,
	/// but the session was not.
	AR_ERROR_NOT_TRACKING = -5,

	/// A texture name was not set by calling ArSession_setCameraTextureName()
	/// before the first call to ArSession_update()
	AR_ERROR_TEXTURE_NOT_SET = -6,

	/// An operation required GL context but one was not available.
	AR_ERROR_MISSING_GL_CONTEXT = -7,

	/// The configuration supplied to ArSession_configure() was unsupported.
	/// To avoid this error, ensure that Session_checkSupported() returns true.
	AR_ERROR_UNSUPPORTED_CONFIGURATION = -8,

	/// The android camera permission has not been granted prior to calling
	/// ArSession_resume()
	AR_ERROR_CAMERA_PERMISSION_NOT_GRANTED = -9,

	/// Acquire failed because the object being acquired is already released.
	/// For example, this happens if the application holds an ::ArFrame beyond
	/// the next call to ArSession_update(), and then tries to acquire its point
	/// cloud.
	AR_ERROR_DEADLINE_EXCEEDED = -10,

	/// There are no available resources to complete the operation.  In cases of
	/// @c acquire methods returning this error, This can be avoided by
	/// releasing previously acquired objects before acquiring new ones.
	AR_ERROR_RESOURCE_EXHAUSTED = -11,

	/// Acquire failed because the data isn't available yet for the current
	/// frame. For example, acquire the image metadata may fail with this error
	/// because the camera hasn't fully started.
	AR_ERROR_NOT_YET_AVAILABLE = -12,

	/// The android camera has been reallocated to a higher priority app or is
	/// otherwise unavailable.
	AR_ERROR_CAMERA_NOT_AVAILABLE = -13,

	/// The data passed in for this operation was not in a valid format.
	AR_ERROR_DATA_INVALID_FORMAT = -18,

	/// The data passed in for this operation is not supported by this version
	/// of the SDK.
	AR_ERROR_DATA_UNSUPPORTED_VERSION = -19,

	/// A function has been invoked at an illegal or inappropriate time. A
	/// message will be printed to logcat with additional details for the
	/// developer.  For example, ArSession_resume() will return this status if
	/// the camera configuration was changed and there are any unreleased
	/// images
	AR_ERROR_ILLEGAL_STATE = -20,

	/// The ARCore APK is not installed on this device.
	AR_UNAVAILABLE_ARCORE_NOT_INSTALLED = -100,

	/// The device is not currently compatible with ARCore.
	AR_UNAVAILABLE_DEVICE_NOT_COMPATIBLE = -101,

	/// The ARCore APK currently installed on device is too old and needs to be
	/// updated.
	AR_UNAVAILABLE_APK_TOO_OLD = -103,

	/// The ARCore APK currently installed no longer supports the ARCore SDK
	/// that the application was built with.
	AR_UNAVAILABLE_SDK_TOO_OLD = -104,

	/// The user declined installation of the ARCore APK during this run of the
	/// application and the current request was not marked as user-initiated.
	AR_UNAVAILABLE_USER_DECLINED_INSTALLATION = -105
};

#if PLATFORM_ANDROID
static ArTrackableType GetTrackableType(UClass* ClassType)
{
	if (ClassType == UARTrackedGeometry::StaticClass())
	{
		return ArTrackableType::AR_TRACKABLE_BASE_TRACKABLE;
	}
	else if (ClassType == UARPlaneGeometry::StaticClass())
	{
		return ArTrackableType::AR_TRACKABLE_PLANE;
	}
	else if (ClassType == UARTrackedPoint::StaticClass())
	{
		return ArTrackableType::AR_TRACKABLE_POINT;
	}
	else if (ClassType == UGoogleARCoreAugmentedImage::StaticClass())
	{
		return ArTrackableType::AR_TRACKABLE_AUGMENTED_IMAGE;
	}
	else if (ClassType == UGoogleARCoreAugmentedFace::StaticClass())
	{
		return ArTrackableType::AR_TRACKABLE_FACE;
	}
	else
	{
		return ArTrackableType::AR_TRACKABLE_NOT_VALID;
	}
}
#endif

class FGoogleARCoreFrame;
class FGoogleARCoreSession;
class UGoogleARCoreCameraImage;

// A wrapper class that stores a native pointer internally, which can be used as the key type for TMap
USTRUCT()
struct FARCorePointer
{
	GENERATED_BODY()
	
	FARCorePointer() = default;
	
	FARCorePointer(void* InPointer) : RawPointer(InPointer) {}
	
	template<class T>
	T* AsRawPointer() const
	{
		return (T*)RawPointer;
	}
	
	friend uint32 GetTypeHash(const FARCorePointer& Pointer)
	{
		return ::GetTypeHash(Pointer.RawPointer);
	}
	
	bool operator==(const FARCorePointer& Other) const
	{
		return RawPointer == Other.RawPointer;
	}

private:
	void* RawPointer = nullptr;
};

UCLASS()
class UGoogleARCoreUObjectManager : public UObject
{
	GENERATED_BODY()

	UGoogleARCoreUObjectManager();
	virtual ~UGoogleARCoreUObjectManager();
	
public:
	UPROPERTY()
	TObjectPtr<UGoogleARCorePointCloud> LatestPointCloud;
	
	// pointer type is ArTrackable*
	UPROPERTY()
	TMap<FARCorePointer, FTrackedGeometryGroup> TrackableHandleMap;
	
	// pointer type is ArAnchor*
	UPROPERTY()
	TMap<FARCorePointer, TObjectPtr<UARPin>> HandleToAnchorMap;
	
	void ClearTrackables();
	
	// Returns the removed geometry
	UARTrackedGeometry* RemoveTrackable(FARCorePointer Pointer);

#if PLATFORM_ANDROID
	const FTrackedGeometryGroup* GetBaseTrackableFromHandle(ArTrackable* TrackableHandle, FGoogleARCoreSession* Session);
	
	template<class T>
	T* GetTrackableFromHandle(ArTrackable* TrackableHandle, FGoogleARCoreSession* Session)
	{
		const FTrackedGeometryGroup* Group = GetBaseTrackableFromHandle(TrackableHandle, Session);
		return Group ? CastChecked<T>(Group->TrackedGeometry) : nullptr;
	}
	
	void DumpTrackableHandleMap(const ArSession* SessionHandle);
	
	void RemoveInvalidTrackables(const TArray<ArTrackable*>& ValidTrackables, TArray<UARPin*>& ARPinsToRemove);
#endif

	UARSessionConfig& AccessSessionConfig();

	void OnSpawnARActor(AARActor* NewARActor, UARComponent* NewARComponent, FGuid NativeID);

	//for networked callbacks
	FDelegateHandle SpawnARActorDelegateHandle;

#if PLATFORM_ANDROID
private:
	//for mapping ArTrackable* to FGuids and back
	FGuid TrackableHandleToGuid(const ArTrackable* TrackableHandle);
	ArTrackable* GuidToTrackableHandle(const FGuid& Guid);
#endif
	FGuid BaseGuid;
};

class FGoogleARCoreAPKManager
{
public:
	static EGoogleARCoreAvailability CheckARCoreAPKAvailability();
	static EGoogleARCoreAPIStatus RequestInstall(bool bUserRequestedInstall, EGoogleARCoreInstallStatus& OutInstallStatus);
};


class FGoogleARCoreSession : public TSharedFromThis<FGoogleARCoreSession>, public FGCObject
{

public:
	static TSharedPtr<FGoogleARCoreSession> CreateARCoreSession();

	FGoogleARCoreSession();
	~FGoogleARCoreSession();

	// Properties
	EGoogleARCoreAPIStatus GetSessionCreateStatus();
	UGoogleARCoreUObjectManager* GetUObjectManager();
	float GetWorldToMeterScale();
	void SetARSystem(TSharedRef<FARSupportInterface, ESPMode::ThreadSafe> InArSystem) { ARSystem = InArSystem; }
	TSharedRef<FARSupportInterface, ESPMode::ThreadSafe> GetARSystem() { return ARSystem.ToSharedRef(); }
#if PLATFORM_ANDROID
	ArSession* GetHandle();
#endif

	// Lifecycle
	bool IsConfigSupported(const UARSessionConfig& Config);
	EGoogleARCoreAPIStatus ConfigSession(const UARSessionConfig& Config);
	const UARSessionConfig* GetCurrentSessionConfig();
	TArray<FGoogleARCoreCameraConfig> GetSupportedCameraConfig(EGoogleARCoreCameraFacing CameraFacing);
	EGoogleARCoreAPIStatus SetCameraConfig(FGoogleARCoreCameraConfig CameraConfig);
	void GetARCameraConfig(FGoogleARCoreCameraConfig& OutCurrentCameraConfig);
	int AddRuntimeAugmentedImage(UGoogleARCoreAugmentedImageDatabase* TargetImageDatabase, const TArray<uint8>& ImageGrayscalePixels,
		int ImageWidth, int ImageHeight, FString ImageName, float ImageWidthInMeter);
	bool AddRuntimeCandidateImage(UARSessionConfig* SessionConfig, const TArray<uint8>& ImageGrayscalePixels,
		int ImageWidth, int ImageHeight, FString FriendlyName, float PhysicalWidth);
	EGoogleARCoreAPIStatus Resume();
	EGoogleARCoreAPIStatus Pause();
	EGoogleARCoreAPIStatus Update(float WorldToMeterScale);
	const FGoogleARCoreFrame* GetLatestFrame();
	uint32 GetFrameNum();

	void SetCameraTextureIds(const TArray<uint32_t>& TextureIds);
	void SetDisplayGeometry(int Rotation, int Width, int Height);

	// Anchor API
	EGoogleARCoreAPIStatus CreateARAnchor(const FTransform& TransfromInTrackingSpace, UARTrackedGeometry* TrackedGeometry, USceneComponent* ComponentToPin, FName InDebugName, UARPin*& OutAnchor);
	bool TryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InPinName, UARPin*& OutPin);
	void DetachAnchor(UARPin* Anchor);

	void GetAllAnchors(TArray<UARPin*>& OutAnchors) const;
	template< class T > void GetAllTrackables(TArray<T*>& OutARCoreTrackableList);

	void* GetLatestFrameRawPointer();
	
	bool IsSceneDepthEnabled() const;

	bool HasNewFrame() const { return bHasNewFrame; }

private:
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FGoogleARCoreSession");
	}

	EGoogleARCoreAPIStatus SessionCreateStatus;
	TObjectPtr<const UARSessionConfig> SessionConfig;
	FGoogleARCoreFrame* LatestFrame;
	TObjectPtr<UGoogleARCoreUObjectManager> UObjectManager;
	float CachedWorldToMeterScale;
	uint32 FrameNumber;

	TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSystem;
#if PLATFORM_ANDROID
	ArSession* SessionHandle = nullptr;
	ArConfig* ConfigHandle = nullptr;
	TMap<const UARSessionConfig*, ArAugmentedImageDatabase*> ImageDatabaseMap;
#endif

	// Stores if current frame has new data. Set on every Update() call.
	bool bHasNewFrame = false;
};

class FGoogleARCoreFrame
{
	friend class FGoogleARCoreSession;

public:
	FGoogleARCoreFrame(FGoogleARCoreSession* Session);
	~FGoogleARCoreFrame();

	void Init();

	void Update(float WorldToMeterScale);

	bool UpdateCameraTexture(UARCoreCameraTexture* InCameraTexture) const;
	void UpdateDepthTexture(UARCoreDepthTexture*& OutDepthTexture) const;

	FTransform GetCameraPose() const;
	int64 GetCameraTimestamp() const;
	EGoogleARCoreTrackingState GetCameraTrackingState() const;
	EGoogleARCoreTrackingFailureReason GetCameraTrackingFailureReason() const;

	void GetUpdatedAnchors(TArray<UARPin*>& OutUpdatedAnchors) const;
	template< class T > void GetUpdatedTrackables(TArray<T*>& OutARCoreTrackableList) const;

	void ARLineTrace(const FVector2D& ScreenPosition, EGoogleARCoreLineTraceChannel RequestedTraceChannels, TArray<FARTraceResult>& OutHitResults) const;
	void ARLineTrace(const FVector& Start, const FVector& End, EGoogleARCoreLineTraceChannel RequestedTraceChannels, TArray<FARTraceResult>& OutHitResults) const;

	bool IsDisplayRotationChanged() const;
	FMatrix GetProjectionMatrix() const;
	void TransformDisplayUvCoords(const TArray<float>& UvCoords, TArray<float>& OutUvCoords) const;

	FGoogleARCoreLightEstimate GetLightEstimate() const;
	EGoogleARCoreAPIStatus GetPointCloud(UGoogleARCorePointCloud*& OutLatestPointCloud) const;
	EGoogleARCoreAPIStatus AcquirePointCloud(UGoogleARCorePointCloud*& OutLatestPointCloud) const;
	EGoogleARCoreAPIStatus AcquireCameraImage(UGoogleARCoreCameraImage *&OutCameraImage) const;
	EGoogleARCoreAPIStatus GetCameraImageIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const;
	EGoogleARCoreAPIStatus GetCameraTextureIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const;

	void TransformARCoordinates2D(EGoogleARCoreCoordinates2DType InputCoordinatesType, const TArray<FVector2D>& InputCoordinates,
		EGoogleARCoreCoordinates2DType OutputCoordinatesType, TArray<FVector2D>& OutputCoordinates) const;
	
	uint32 GetCameraTextureId() const { return CameraTextureId; }
	
#if PLATFORM_ANDROID
	ArFrame* GetHandle() { return FrameHandle; };
	static TArray<ArTrackable*> GetTrackables(const ArSession* SessionHandle, ArTrackableList* ListHandle, bool bRemoveSubsumedPlanes);
#endif

private:
	FGoogleARCoreSession* Session;
	FTransform LatestCameraPose;
	int64 LatestCameraTimestamp;
	uint32 CameraTextureId = 0;
	EGoogleARCoreTrackingState LatestCameraTrackingState;
	EGoogleARCoreTrackingFailureReason LatestCameraTrackingFailureReason;

	EGoogleARCoreAPIStatus LatestPointCloudStatus;
	EGoogleARCoreAPIStatus LatestImageMetadataStatus;

	TArray<UARPin*> UpdatedAnchors;

#if PLATFORM_ANDROID
	const ArSession* SessionHandle = nullptr;
	ArFrame* FrameHandle = nullptr;
	ArCamera* CameraHandle = nullptr;
	ArPose* SketchPoseHandle = nullptr;
	ArImageMetadata* LatestImageMetadata = nullptr;

	void* VulkanHardwareBuffer = nullptr; // This is used instead of CameraTextureId when using Vulkan

	void FilterLineTraceResults(ArHitResultList* HitResultList, EGoogleARCoreLineTraceChannel RequestedTraceChannels,
		TArray<FARTraceResult>& OutHitResults, float MaxDistance = TNumericLimits<float>::Max()) const;
#endif
};

template< class T >
void FGoogleARCoreFrame::GetUpdatedTrackables(TArray<T*>& OutARCoreTrackableList) const
{
	OutARCoreTrackableList.Empty();
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return;
	}

	ArTrackableType TrackableType = GetTrackableType(T::StaticClass());
	if (TrackableType == ArTrackableType::AR_TRACKABLE_NOT_VALID)
	{
		UE_LOGF(LogGoogleARCoreAPI, Error, "Invalid Trackable type: %ls", *T::StaticClass()->GetName());
		return;
	}

	ArTrackableList* TrackableListHandle = nullptr;
	ArTrackableList_create(SessionHandle, &TrackableListHandle);
	ArFrame_getUpdatedTrackables(SessionHandle, FrameHandle, TrackableType, TrackableListHandle);
	auto Trackables = GetTrackables(SessionHandle, TrackableListHandle, true);
	for (auto TrackableHandle : Trackables)
	{
		T* TrackableObject = Session->GetUObjectManager()->template GetTrackableFromHandle<T>(TrackableHandle, Session);
		if (TrackableObject)
		{
			OutARCoreTrackableList.Add(TrackableObject);
		}
	}
	ArTrackableList_destroy(TrackableListHandle);
#endif
}

template< class T >
void FGoogleARCoreSession::GetAllTrackables(TArray<T*>& OutARCoreTrackableList)
{
	OutARCoreTrackableList.Empty();
#if PLATFORM_ANDROID
	if (SessionHandle == nullptr)
	{
		return;
	}

	ArTrackableType TrackableType = GetTrackableType(T::StaticClass());
	if (TrackableType == ArTrackableType::AR_TRACKABLE_NOT_VALID)
	{
		UE_LOGF(LogGoogleARCoreAPI, Error, "Invalid Trackable type: %ls", *T::StaticClass()->GetName());
		return;
	}

	ArTrackableList* TrackableListHandle = nullptr;
	ArTrackableList_create(SessionHandle, &TrackableListHandle);
	ArSession_getAllTrackables(SessionHandle, TrackableType, TrackableListHandle);
	
	auto Trackables = FGoogleARCoreFrame::GetTrackables(SessionHandle, TrackableListHandle, true);
	for (auto TrackableHandle : Trackables)
	{
		T* TrackableObject = UObjectManager->template GetTrackableFromHandle<T>(TrackableHandle, this);
		if (TrackableObject)
		{
			OutARCoreTrackableList.Add(TrackableObject);
		}
	}
	ArTrackableList_destroy(TrackableListHandle);
#endif
}
