// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreAndroidHelper.h"
#include "GoogleARCoreBaseLogCategory.h"
#if PLATFORM_ANDROID
#include "Android/AndroidJNI.h"
#endif

#include "GoogleARCoreDevice.h"

#if PLATFORM_ANDROID
extern "C"
{
	// Functions that are called on Android lifecycle events.

	JNI_METHOD void Java_com_google_arcore_unreal_GoogleARCoreJavaHelper_onApplicationCreated(JNIEnv*, jobject)
	{
		FGoogleARCoreAndroidHelper::OnApplicationCreated();
	}

	JNI_METHOD void Java_com_google_arcore_unreal_GoogleARCoreJavaHelper_onApplicationDestroyed(JNIEnv*, jobject)
	{
		FGoogleARCoreAndroidHelper::OnApplicationDestroyed();
	}

	JNI_METHOD void Java_com_google_arcore_unreal_GoogleARCoreJavaHelper_onApplicationPause(JNIEnv*, jobject)
	{
		FGoogleARCoreAndroidHelper::OnApplicationPause();
	}

	JNI_METHOD void Java_com_google_arcore_unreal_GoogleARCoreJavaHelper_onApplicationResume(JNIEnv*, jobject)
	{
		FGoogleARCoreAndroidHelper::OnApplicationResume();
	}

	JNI_METHOD void Java_com_google_arcore_unreal_GoogleARCoreJavaHelper_onApplicationStop(JNIEnv*, jobject)
	{
		FGoogleARCoreAndroidHelper::OnApplicationStop();
	}

	JNI_METHOD void Java_com_google_arcore_unreal_GoogleARCoreJavaHelper_onApplicationStart(JNIEnv*, jobject)
	{
		FGoogleARCoreAndroidHelper::OnApplicationStart();
	}

	JNI_METHOD void Java_com_google_arcore_unreal_GoogleARCoreJavaHelper_onDisplayOrientationChanged(JNIEnv*, jobject)
	{
		FGoogleARCoreAndroidHelper::OnDisplayOrientationChanged();
	}

	JNI_METHOD void Java_com_google_arcore_unreal_GoogleARCoreJavaHelper_ARCoreSessionStart(JNIEnv*, jobject)
	{
		FGoogleARCoreDevice::GetInstance()->StartSessionWithRequestedConfig();
	}
}

// Redirect events to FGoogleARCoreDevice class:

void FGoogleARCoreAndroidHelper::OnApplicationCreated()
{
	FGoogleARCoreDevice::GetInstance()->OnApplicationCreated();
}

void FGoogleARCoreAndroidHelper::OnApplicationDestroyed()
{
	FGoogleARCoreDevice::GetInstance()->OnApplicationDestroyed();
}

void FGoogleARCoreAndroidHelper::OnApplicationPause()
{
	FGoogleARCoreDevice::GetInstance()->OnApplicationPause();
}

void FGoogleARCoreAndroidHelper::OnApplicationStart()
{
	FGoogleARCoreDevice::GetInstance()->OnApplicationStart();
}

void FGoogleARCoreAndroidHelper::OnApplicationStop()
{
	FGoogleARCoreDevice::GetInstance()->OnApplicationStop();
}

void FGoogleARCoreAndroidHelper::OnApplicationResume()
{
	FGoogleARCoreDevice::GetInstance()->OnApplicationResume();
}

void FGoogleARCoreAndroidHelper::OnDisplayOrientationChanged()
{
	FGoogleARCoreDevice::GetInstance()->OnDisplayOrientationChanged();
}

#endif

ARCoreDisplayRotation FGoogleARCoreAndroidHelper::CurrentDisplayRotation = ARCoreDisplayRotation::Rotation0;
void FGoogleARCoreAndroidHelper::UpdateDisplayRotation()
{
#if PLATFORM_ANDROID
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_GetDisplayRotation", "()I", false);
		int32 IntCurrentDisplayRotation = FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, Method);
		CurrentDisplayRotation = static_cast<ARCoreDisplayRotation>(IntCurrentDisplayRotation);
		check((CurrentDisplayRotation >= ARCoreDisplayRotation::Rotation0) || (CurrentDisplayRotation <= ARCoreDisplayRotation::Max));
	}
#endif
}

ARCoreDisplayRotation FGoogleARCoreAndroidHelper::GetDisplayRotation()
{
	return CurrentDisplayRotation;
}

void FGoogleARCoreAndroidHelper::QueueStartSessionOnUiThread()
{
#if PLATFORM_ANDROID
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_QueueStartSessionOnUiThread", "()V", false);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method);
	}
#endif
}
