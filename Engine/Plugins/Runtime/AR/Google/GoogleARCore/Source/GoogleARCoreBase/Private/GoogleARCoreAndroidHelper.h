// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if PLATFORM_ANDROID
#include "Android/AndroidApplication.h"
#endif

// Matches return values from android.view.display.getRotation()
enum class ARCoreDisplayRotation : int32
{
	Rotation0 = 0,
	Rotation90 = 1,
	Rotation180 = 2,
	Rotation270 = 3,
	Max = 3
};

/** Wrappers for accessing Android Java stuff */
class FGoogleARCoreAndroidHelper
{
public:

	/**
	 * Update the Android display orientation as per the android.view.Display class' getRotation() method.
	 */
	static void UpdateDisplayRotation();

	/**
	 * Get Andriod display orientation.
	 */
	static ARCoreDisplayRotation GetDisplayRotation();

	static void QueueStartSessionOnUiThread();

#if PLATFORM_ANDROID
	// Helpers for redirecting Android events.
	static void OnApplicationCreated();
	static void OnApplicationDestroyed();
	static void OnApplicationPause();
	static void OnApplicationResume();
	static void OnApplicationStop();
	static void OnApplicationStart();
	static void OnDisplayOrientationChanged();
#endif

private:
	static ARCoreDisplayRotation CurrentDisplayRotation;
};
