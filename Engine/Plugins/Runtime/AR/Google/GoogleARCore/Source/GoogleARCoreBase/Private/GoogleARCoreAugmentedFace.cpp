// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreAugmentedFace.h"

#include "IXRTrackingSystem.h"
#include "ARSystem.h"
#include "ARSupportInterface.h"

void UGoogleARCoreAugmentedFace::DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds /*= 0.0f*/) const
{
	auto Indices = GetIndexBuffer();
	auto Vertices = GetVertexBuffer();

	FTransform WorldTransform = GetLocalToWorldTransform();

	for (int i = 0; i < Indices.Num(); i += 3)
	{
		FVector V1 = WorldTransform.TransformPosition(Vertices[Indices[i]]);
		FVector V2 = WorldTransform.TransformPosition(Vertices[Indices[i + 1]]);
		FVector V3 = WorldTransform.TransformPosition(Vertices[Indices[i + 2]]);

		DrawDebugLine(World, V1, V2, OutlineColor.ToFColor(false), false, PersistForSeconds, 0, 0.1 * OutlineThickness);
		DrawDebugLine(World, V2, V3, OutlineColor.ToFColor(false), false, PersistForSeconds, 0, 0.1 * OutlineThickness);
		DrawDebugLine(World, V3, V1, OutlineColor.ToFColor(false), false, PersistForSeconds, 0, 0.1 * OutlineThickness);
	}
}

FTransform UGoogleARCoreAugmentedFace::GetLocalToWorldTransformOfRegion(EGoogleARCoreAugmentedFaceRegion Region)
{
	if (RegionLocalToTrackingTransforms.Contains(Region))
	{
		return RegionLocalToTrackingTransforms[Region] * GetARSystem()->GetAlignmentTransform() * GetARSystem()->GetXRTrackingSystem()->GetTrackingToWorldTransform();
	}

	return FTransform::Identity;
}

FTransform UGoogleARCoreAugmentedFace::GetLocalToTrackingTransformOfRegion(EGoogleARCoreAugmentedFaceRegion Region)
{
	if(RegionLocalToTrackingTransforms.Contains(Region))
	{
		return RegionLocalToTrackingTransforms[Region];
	}

	return FTransform::Identity;
}

void UGoogleARCoreAugmentedFace::UpdateRegionTransforms(TMap<EGoogleARCoreAugmentedFaceRegion, FTransform>& InRegionLocalToTrackingTransforms)
{
	RegionLocalToTrackingTransforms = MoveTemp(InRegionLocalToTrackingTransforms);
}
