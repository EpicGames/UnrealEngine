// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreAugmentedImage.h"

#include "GoogleARCoreAugmentedImageDatabase.h"
#include "GoogleARCoreDevice.h"
#include "GoogleARCoreAPI.h"

#include "CoreMinimal.h"
#include "DrawDebugHelpers.h"

#if PLATFORM_ANDROID
#include "arcore_c_api.h"
#endif

void UGoogleARCoreAugmentedImage::DebugDraw(
	UWorld* World, const FLinearColor& OutlineColor,
	float OutlineThickness, float PersistForSeconds) const
{
#if PLATFORM_ANDROID

	FTransform CenterTransform =
		GetLocalToTrackingTransform();

	FVector2D HalfExtent(EstimatedSize / 2.0f);
	FVector Corner0(-HalfExtent.Y, -HalfExtent.X, 0.0f);
	FVector Corner1( HalfExtent.Y, -HalfExtent.X, 0.0f);
	FVector Corner2( HalfExtent.Y,  HalfExtent.X, 0.0f);
	FVector Corner3(-HalfExtent.Y,  HalfExtent.X, 0.0f);

	DrawDebugLine(
		World,
		CenterTransform.TransformPosition(Corner0),
		CenterTransform.TransformPosition(Corner1),
		OutlineColor.ToFColor(false),
		false,
		PersistForSeconds,
		0, OutlineThickness);

	DrawDebugLine(
		World,
		CenterTransform.TransformPosition(Corner1),
		CenterTransform.TransformPosition(Corner2),
		OutlineColor.ToFColor(false),
		false,
		PersistForSeconds,
		0, OutlineThickness);

	DrawDebugLine(
		World,
		CenterTransform.TransformPosition(Corner2),
		CenterTransform.TransformPosition(Corner3),
		OutlineColor.ToFColor(false),
		false,
		PersistForSeconds,
		0, OutlineThickness);

	DrawDebugLine(
		World,
		CenterTransform.TransformPosition(Corner3),
		CenterTransform.TransformPosition(Corner0),
		OutlineColor.ToFColor(false),
		false,
		PersistForSeconds,
		0, OutlineThickness);

#endif
}

void UGoogleARCoreAugmentedImage::UpdateTrackedGeometry(
	const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem,
	uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform,
	const FTransform& InAlignmentTransform,
	FVector2D InEstimatedSize, UARCandidateImage* InDetectedImage,
	int32 InImageIndex, const FString& InImageName)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform, InEstimatedSize, InDetectedImage);

	ImageIndex = InImageIndex;
	ImageName = InImageName;
}
