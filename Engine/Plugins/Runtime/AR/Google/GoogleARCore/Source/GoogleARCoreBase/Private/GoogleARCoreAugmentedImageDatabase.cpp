// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreAugmentedImageDatabase.h"

#include "GoogleARCoreAPI.h"
#include "GoogleARCoreBaseLogCategory.h"
#include "GoogleARCoreCookSupport.h"
#include "GoogleARCoreDevice.h"

#include "Kismet/GameplayStatics.h"
#include "Misc/FileHelper.h"
#include "Misc/AssertionMacros.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Containers/StringConv.h"
#include "RenderUtils.h"

#include "png.h"
#include "TextureResource.h"

void UGoogleARCoreAugmentedImageDatabase::Serialize(FArchive& Ar)
{
	FString OutStdout;
	FString OutStderr;
	int32 OutReturnCode = 0;

#if !PLATFORM_ANDROID && WITH_EDITORONLY_DATA

	if (!Ar.IsLoading() && Ar.IsCooking()) {
		SerializedDatabase.Empty();

		if (Entries.Num()) {

			FString PathToDbTool =
				FPaths::Combine(
					*FPaths::EnginePluginsDir(),
					TEXT("Runtime"),
					TEXT("AR"),
					TEXT("Google"),
					TEXT("GoogleARCore"),
					TEXT("Binaries"),
					TEXT("ThirdParty"),
					TEXT("Google"),
					TEXT("ARCoreImg"),
					*UGameplayStatics::GetPlatformName(),
#if PLATFORM_LINUX
					TEXT("arcoreimg")
#elif PLATFORM_WINDOWS
					TEXT("arcoreimg.exe")
#elif PLATFORM_MAC
					TEXT("ptdbtool_macos_lipobin")
#endif
					);

			FString TempDir =
				FPaths::ConvertRelativePathToFull(
					FPaths::Combine(
						*FPaths::EnginePluginsDir(),
						TEXT("Runtime"),
						TEXT("AR"),
						TEXT("Google"),
						TEXT("GoogleARCore"),
						TEXT("Intermediate"),
						TEXT("ARCoreImgTemp")));

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (!PlatformFile.DirectoryExists(*TempDir))
			{
				PlatformFile.CreateDirectory(*TempDir);
			}

			TArray<FString> CleanupList;

			FString ImageListFileContents;

			for (int32 i = 0; i < Entries.Num(); i++) {
				UTexture2D *Tex = Entries[i].ImageAsset;

				if (Tex) {

					FString PNGFilename =
						FPaths::Combine(TempDir, Tex->GetName() + FString(".png"));

					if (FGoogleARCoreSessionConfigCookSupport::SaveTextureToPNG(
							Tex,
							PNGFilename))
					{
						// "|" is used as a delimeter in the image
						// list, and there doesn't seem to be any way
						// to escape them, so they will be replaced
						// with underscores.
						FString TmpName =
							Entries[i].Name.ToString().Replace(TEXT("|"), TEXT("_"));

						ImageListFileContents +=
							TmpName +
							FString("|") +
							PNGFilename;

						CleanupList.Add(PNGFilename);

						if (Entries[i].Width > 0.0f) {
							ImageListFileContents +=
								FString("|") +
								FString::SanitizeFloat(Entries[i].Width);
						}

						ImageListFileContents += FString("\n");
					}

				}
			}

			FString PathToImageList =
				FPaths::Combine(TempDir, TEXT("image_list.txt"));
			FString PathToImageDb =
				FPaths::Combine(TempDir, TEXT("image_list.imgdb"));

			CleanupList.Add(PathToImageList);
			CleanupList.Add(PathToImageDb);

			FFileHelper::SaveStringToFile(
				ImageListFileContents,
				*PathToImageList);

			OutStderr = "";
			OutStdout = "";
			OutReturnCode = 0;

#if PLATFORM_LINUX || PLATFORM_MAC
			FGoogleARCoreSessionConfigCookSupport::PlatformSetExecutable(*PathToDbTool, true);
#endif

			FPlatformProcess::ExecProcess(
				*PathToDbTool,
				*(FString("build-db --input_image_list_path=\"")+ PathToImageList +
				  FString("\" --output_db_path=\"") + PathToImageDb + "\""),
				&OutReturnCode,
				&OutStdout,
				&OutStderr);

			if (OutReturnCode)
			{
				Ar.SetCriticalError();
				UE_LOGF(LogGoogleARCoreAPI, Error, "Failed to build augmented image database: %ls", *OutStderr);
			}
			else
			{
				FFileHelper::LoadFileToArray(SerializedDatabase, *PathToImageDb, 0);
				UE_LOGF(LogGoogleARCoreAPI, Log,
					   "Augmented image database created. Size: %d bytes. Tool output: %ls",
					   SerializedDatabase.Num(), *OutStdout);
			}

			for (int32 i = 0; i < CleanupList.Num(); i++)
			{
				UE_LOGF(LogGoogleARCoreAPI, Log, "Cleaning up: %ls", *CleanupList[i]);
				PlatformFile.DeleteFile(*CleanupList[i]);
			}

			PlatformFile.DeleteDirectory(*TempDir);
		}
	}

#endif

	// Must happen AFTER database generation, because we rely on the
	// UPROPERTY serialization to actually save the data.
	Super::Serialize(Ar);
}
