// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GoogleARCoreBaseModule.h"

DEFINE_LOG_CATEGORY_STATIC(LogGoogleARCore, Log, All);
