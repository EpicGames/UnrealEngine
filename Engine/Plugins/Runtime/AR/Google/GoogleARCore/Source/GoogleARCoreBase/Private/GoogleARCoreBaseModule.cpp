// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreBaseModule.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "ARSessionConfig.h"

#include "GoogleARCoreMotionController.h"
#include "GoogleARCoreXRTrackingSystem.h"
#include "GoogleARCoreDevice.h"
#include "GoogleARCoreCookSupport.h"
#include "GoogleARCoreBaseLogCategory.h"

#define LOCTEXT_NAMESPACE "GoogleARCore"

#if PLATFORM_ANDROID
#include "IVulkanDynamicRHI.h"
#endif

class FGoogleARCoreBaseModule : public IGoogleARCoreBaseModule
{
public:
	/** IHeadMountedDisplayModule implementation*/
	/** Returns the key into the HMDPluginPriority section of the config file for this module */
	virtual FString GetModuleKeyName() const override
	{
		return TEXT("GoogleARCoreHMD");
	}

	/**
	 *
	 */
	virtual bool IsHMDConnected()
	{
		// @todo arcore do we have an API for querying this?
		return true;
	}

	/**
	 * Attempts to create a new head tracking device interface
	 *
	 * @return	Interface to the new head tracking device, if we were able to successfully create one
	 */
	virtual TSharedPtr< class IXRTrackingSystem, ESPMode::ThreadSafe > CreateTrackingSystem() override;

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FGoogleARCoreMotionController ControllerInstance;

#if WITH_EDITORONLY_DATA
	FGoogleARCoreSessionConfigCookSupport SessionConfigCookSupport;
#endif

};

IMPLEMENT_MODULE(FGoogleARCoreBaseModule, GoogleARCoreBase)

TSharedPtr< class IXRTrackingSystem, ESPMode::ThreadSafe > FGoogleARCoreBaseModule::CreateTrackingSystem()
{
#if PLATFORM_ANDROID
	auto ARCoreSystem = MakeShared<FGoogleARCoreXRTrackingSystem, ESPMode::ThreadSafe>();
	ARCoreSystem->GetARCompositionComponent()->InitializeARSystem();
	FGoogleARCoreDevice::GetInstance()->SetARSystem(ARCoreSystem->GetARCompositionComponent());
	FGenericCrashContext::SetEngineData(TEXT("XR"), TEXT("XR GoogleARCore"));
	return ARCoreSystem;
#else
	return TSharedPtr<class IXRTrackingSystem, ESPMode::ThreadSafe>();
#endif
}

void FGoogleARCoreBaseModule::StartupModule()
{
#if PLATFORM_ANDROID
	if (FAndroidMisc::ShouldUseVulkan())
	{
		IVulkanDynamicRHI* RHI = GetIVulkanDynamicRHI();
		TArray<FAnsiString> LoadedDeviceExtensions = RHI->RHIGetLoadedDeviceExtensions();
		if (!LoadedDeviceExtensions.Find(FAnsiString(VK_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER_EXTENSION_NAME)))
		{
			UE_LOG(LogGoogleARCore, Error, TEXT("Required Vulkan Device extension '" VK_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER_EXTENSION_NAME "' is not available!"));
		}
		if (!LoadedDeviceExtensions.Find(FAnsiString(VK_EXT_QUEUE_FAMILY_FOREIGN_EXTENSION_NAME)))
		{
			UE_LOG(LogGoogleARCore, Error, TEXT("Required Vulkan Device extension '" VK_EXT_QUEUE_FAMILY_FOREIGN_EXTENSION_NAME "' is not available!"));
		}
		if (!LoadedDeviceExtensions.Find(FAnsiString(VK_KHR_SAMPLER_YCBCR_CONVERSION_EXTENSION_NAME)))
		{
			UE_LOG(LogGoogleARCore, Error, TEXT("Required Vulkan Device extension '" VK_KHR_SAMPLER_YCBCR_CONVERSION_EXTENSION_NAME "' is not available!"));
		}
	}
#endif

	ensureMsgf(FModuleManager::Get().LoadModule("AugmentedReality"), TEXT("ARCore depends on the AugmentedReality module.") );

	// Register editor settings:
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "GoogleARCore",
			LOCTEXT("GoogleARCoreSetting", "GoogleARCore"),
			LOCTEXT("GoogleARCoreSettingDescription", "Settings of the GoogleARCore plugin"),
			GetMutableDefault<UGoogleARCoreEditorSettings>());
	}

	// Complete ARCore setup.
	FGoogleARCoreDevice::GetInstance()->OnModuleLoaded();

	// Register VR-like controller interface.
	ControllerInstance.RegisterController();

#if WITH_EDITORONLY_DATA
	SessionConfigCookSupport.RegisterModuleFeature();
#endif

	// Register IHeadMountedDisplayModule
	IHeadMountedDisplayModule::StartupModule();

	FModuleManager::LoadModulePtr<IModuleInterface>("AugmentedReality");
}

void FGoogleARCoreBaseModule::ShutdownModule()
{
	// Unregister IHeadMountedDisplayModule
	IHeadMountedDisplayModule::ShutdownModule();

	// Unregister VR-like controller interface.
	ControllerInstance.UnregisterController();

#if WITH_EDITORONLY_DATA
	SessionConfigCookSupport.UnregisterModuleFeature();
#endif

	// Complete ARCore teardown.
	FGoogleARCoreDevice::GetInstance()->OnModuleUnloaded();

	// Unregister editor settings.
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "GoogleARCore");
	}
}

#undef LOCTEXT_NAMESPACE
