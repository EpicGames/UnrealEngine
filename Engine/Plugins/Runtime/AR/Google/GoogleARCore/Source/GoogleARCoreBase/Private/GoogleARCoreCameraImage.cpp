// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreCameraImage.h"

#include "GoogleARCoreAPI.h"

#if PLATFORM_ANDROID
#include "Android/AndroidApplication.h"
#include "Android/AndroidJNI.h"
#include "Ndk/NdkImageAPI.h"
#endif

UGoogleARCoreCameraImage::~UGoogleARCoreCameraImage()
{
	Release();
}

void UGoogleARCoreCameraImage::Release()
{
#if PLATFORM_ANDROID
	if (ArImage)
	{
		ArImage_release(ArImage);
		ArImage = nullptr;
	}
#endif
}

int32 UGoogleARCoreCameraImage::GetWidth() const
{
	int32_t Width = 0;
#if PLATFORM_ANDROID
	if (ArImage && SessionHandle)
	{
		ArImage_getWidth(SessionHandle, ArImage, &Width);
	}
#endif
	return Width;
}

int32 UGoogleARCoreCameraImage::GetHeight() const
{
	int32_t Height = 0;
#if PLATFORM_ANDROID
	if (ArImage && SessionHandle)
	{
		ArImage_getHeight(SessionHandle, ArImage, &Height);
	}
#endif
	return Height;
}


int32 UGoogleARCoreCameraImage::GetPlaneCount() const
{
	int32_t PlaneCount = 0;
#if PLATFORM_ANDROID
	if (ArImage && SessionHandle)
	{
		ArImage_getNumberOfPlanes(SessionHandle, ArImage, &PlaneCount);
	}
#endif
	return PlaneCount;
}

const uint8 *UGoogleARCoreCameraImage::GetPlaneData(
	int32 Plane, int32 &PixelStride,
	int32 &RowStride, int32 &DataLength) const
{
	const uint8_t *PlaneData = nullptr;
#if PLATFORM_ANDROID
	if (ArImage && SessionHandle)
	{
		ArImage_getPlanePixelStride(SessionHandle, ArImage, Plane, &PixelStride);
		ArImage_getPlaneRowStride(SessionHandle, ArImage, Plane, &RowStride);
		ArImage_getPlaneData(SessionHandle, ArImage, Plane, &PlaneData, &DataLength);
	}
#endif
	return PlaneData;
}
