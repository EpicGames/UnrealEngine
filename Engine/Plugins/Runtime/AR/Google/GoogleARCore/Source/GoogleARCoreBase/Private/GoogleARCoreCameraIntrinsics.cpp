// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreCameraIntrinsics.h"

#if PLATFORM_ANDROID
#include "arcore_c_api.h"
#endif

#include "GoogleARCoreAPI.h"

void UGoogleARCoreCameraIntrinsics::GetFocalLength(float &OutFX, float &OutFY)
{
	OutFX = CameraIntrinsics.FocalLength.X;
	OutFY = CameraIntrinsics.FocalLength.Y;
}

void UGoogleARCoreCameraIntrinsics::GetPrincipalPoint(float &OutCX, float &OutCY)
{
	OutCX = CameraIntrinsics.PrincipalPoint.X;
	OutCY = CameraIntrinsics.PrincipalPoint.Y;
}

void UGoogleARCoreCameraIntrinsics::GetImageDimensions(int32 &OutWidth, int32 &OutHeight)
{
	OutWidth = CameraIntrinsics.ImageResolution.X;
	OutHeight = CameraIntrinsics.ImageResolution.Y;
}

void UGoogleARCoreCameraIntrinsics::SetCameraIntrinsics(const FARCameraIntrinsics& InCameraIntrinsics)
{
	CameraIntrinsics = InCameraIntrinsics;
}
