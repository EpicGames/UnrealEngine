// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreCameraTextureResourceVulkan.h"
#include "GoogleARCoreBaseLogCategory.h"
#include "ExternalTexture.h"

#if PLATFORM_ANDROID
#include "VulkanCommon.h"
#include "IVulkanDynamicRHI.h"
#endif

FARCoreCameraTextureResourceVulkan::FARCoreCameraTextureResourceVulkan(const FGuid& InExternalTextureGuid, void* InHardwareBuffer, uint32 InSizeX, uint32 InSizeY)
	: ExternalTextureGuid(InExternalTextureGuid)
	, SizeX(InSizeX)
	, SizeY(InSizeY)
	, HardwareBuffer(InHardwareBuffer)
{
	// With Vulkan we don't use the Id from the native resource because it gets recreated every frame.
	// We handle the binding of the hardware buffer to the texture ourselves. Using a hash of the external
	// texture Guid is sufficient and will remain the same from frame to frame.
	TextureId = GetTypeHash(ExternalTextureGuid);
}

void FARCoreCameraTextureResourceVulkan::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (HardwareBuffer)
	{
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
		SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);

#if PLATFORM_ANDROID
		IVulkanDynamicRHI* VulkanRHI = GetIVulkanDynamicRHI();
		TextureRHI = VulkanRHI->RHICreateTexture2DFromAndroidHardwareBuffer(reinterpret_cast<AHardwareBuffer*>(HardwareBuffer));
#endif

		FillCameraYCbCrConversionParameters();

		FExternalTextureRegistry::Get().RegisterExternalTexture(ExternalTextureGuid, TextureRHI, SamplerStateRHI);

	}
}

void FARCoreCameraTextureResourceVulkan::ReleaseRHI()
{
	if (HardwareBuffer)
	{
		FExternalTextureRegistry::Get().UnregisterExternalTexture(ExternalTextureGuid);
		FTextureResource::ReleaseRHI();
	}
}

#if PLATFORM_ANDROID
namespace VulkanUtil
{
	EYCbCrModelConversion ToEYCbCrModelConversion(VkSamplerYcbcrModelConversion VulkanYcbcrModelConversion)
	{
		switch (VulkanYcbcrModelConversion)
		{
		case VK_SAMPLER_YCBCR_MODEL_CONVERSION_RGB_IDENTITY:   return EYCbCrModelConversion::None;
		case VK_SAMPLER_YCBCR_MODEL_CONVERSION_YCBCR_IDENTITY: return EYCbCrModelConversion::YCbCrIdentity;
		case VK_SAMPLER_YCBCR_MODEL_CONVERSION_YCBCR_709:      return EYCbCrModelConversion::YCbCrRec709;
		case VK_SAMPLER_YCBCR_MODEL_CONVERSION_YCBCR_601:      return EYCbCrModelConversion::YCbCrRec601;
		case VK_SAMPLER_YCBCR_MODEL_CONVERSION_YCBCR_2020:     return EYCbCrModelConversion::YCbCrRec2020;

		default:
			UE_LOGF(LogGoogleARCore, Error, "Unexpected Vulkan Ycbcr Model Conversion value: %d", VulkanYcbcrModelConversion);
			return EYCbCrModelConversion::None;
		}
	}

	EYCbCrRange ToEYCbCrRange(VkSamplerYcbcrRange VulkanYcbcrRange)
	{
		switch (VulkanYcbcrRange)
		{
		case VK_SAMPLER_YCBCR_RANGE_ITU_FULL:   return EYCbCrRange::Full;
		case VK_SAMPLER_YCBCR_RANGE_ITU_NARROW: return EYCbCrRange::Narrow;

		default:
			UE_LOGF(LogGoogleARCore, Error, "Unexpected Vulkan Ycbcr Range value: %d", VulkanYcbcrRange);
			return EYCbCrRange::Unknown;
		}
	}
}
#endif

void FARCoreCameraTextureResourceVulkan::FillCameraYCbCrConversionParameters()
{
#if PLATFORM_ANDROID
	IVulkanDynamicRHI* VulkanRHI = GetIVulkanDynamicRHI();
	VkDevice VulkanDevice = VulkanRHI->RHIGetVkDevice();

	VkAndroidHardwareBufferFormatPropertiesANDROID HardwareBufferFormatProperties;
	ZeroVulkanStruct(HardwareBufferFormatProperties, VK_STRUCTURE_TYPE_ANDROID_HARDWARE_BUFFER_FORMAT_PROPERTIES_ANDROID);

	VkAndroidHardwareBufferPropertiesANDROID HardwareBufferProperties;
	ZeroVulkanStruct(HardwareBufferProperties, VK_STRUCTURE_TYPE_ANDROID_HARDWARE_BUFFER_PROPERTIES_ANDROID);
	HardwareBufferProperties.pNext = &HardwareBufferFormatProperties;

	PFN_vkGetAndroidHardwareBufferPropertiesANDROID GetAndroidHardwareBufferPropertiesANDROID = (PFN_vkGetAndroidHardwareBufferPropertiesANDROID)VulkanRHI->RHIGetVkDeviceProcAddr("vkGetAndroidHardwareBufferPropertiesANDROID");
	if (!GetAndroidHardwareBufferPropertiesANDROID)
	{
		UE_LOGF(LogGoogleARCore, Error, "PFN_vkGetAndroidHardwareBufferPropertiesANDROID address not found.");
		return;
	}
	VERIFYVULKANRESULT_EXTERNAL(GetAndroidHardwareBufferPropertiesANDROID(VulkanDevice, reinterpret_cast<AHardwareBuffer*>(HardwareBuffer), &HardwareBufferProperties));

	CameraYCbCrConversion.YCbCrModelConversion = VulkanUtil::ToEYCbCrModelConversion(HardwareBufferFormatProperties.suggestedYcbcrModel);
	CameraYCbCrConversion.YCbCrRange = VulkanUtil::ToEYCbCrRange(HardwareBufferFormatProperties.suggestedYcbcrRange);
	CameraYCbCrConversion.NumBits = 8; // Defaulting to 8 bits per component. Investigate how to obtain it from HardwareBuffer.
#endif
}
