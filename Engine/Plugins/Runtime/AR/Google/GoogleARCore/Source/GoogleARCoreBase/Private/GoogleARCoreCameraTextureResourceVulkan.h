// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GoogleARCoreYCbCrConversion.h"
#include "TextureResource.h"
#include "RHICommandList.h"
#include "Misc/Guid.h"

// Camera texture resource using the Vulkan hardware buffer provided.
// 
// Known Vulkan Validation error: VUID-VkWriteDescriptorSet-descriptorType-01946
//
// Vulkan RHI will create the image and its view with a VkSamplerYcbcrConversion, and bind it as a VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE.
// But using a VkSamplerYcbcrConversion requires the sampler and the image view to be bound together with a
// VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER, which is not supported.
// 
// As a workaround we perform the YCbCr conversion ourselves in the shader and do not pass VkSamplerYcbcrConversion
// to the sampler. The VkSamplerYcbcrConversion is still necessary for the view, which reports the validation error,
// and it will read the values from the external image. In the future if support for combined image sampler descriptors
// is provided, then it would possible to use VkSamplerYcbcrConversion natively and remove the YCbCr conversion inside
// the shader.
class FARCoreCameraTextureResourceVulkan : public FTextureResource
{
public:
	FARCoreCameraTextureResourceVulkan(const FGuid& InExternalTextureGuid, void* InHardwareBuffer, uint32 InSizeX, uint32 InSizeY);

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	virtual uint32 GetSizeX() const override
	{
		return SizeX;
	}

	virtual uint32 GetSizeY() const override
	{
		return SizeY;
	}

	uint32 GetTextureId() const
	{
		return TextureId;
	}

	FYCbCrConversion GetCameraYCbCrConversion() const
	{
		return CameraYCbCrConversion;
	}

private:
	void FillCameraYCbCrConversionParameters();

	uint32 TextureId = 0;
	const FGuid ExternalTextureGuid;
	const uint32 SizeX;
	const uint32 SizeY;

	void* HardwareBuffer = nullptr;
	FYCbCrConversion CameraYCbCrConversion;
};
