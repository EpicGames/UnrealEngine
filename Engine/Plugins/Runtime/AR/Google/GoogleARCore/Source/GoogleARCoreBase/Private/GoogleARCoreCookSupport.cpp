// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreCookSupport.h"
#include "GoogleARCoreBaseLogCategory.h"

#include "Engine/Texture2D.h"
#include "Interfaces/ITargetPlatform.h"
#include "ARTypes.h"
#include "ARSessionConfig.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/FileHelper.h"
#include "Misc/AssertionMacros.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "RenderUtils.h"

#include "png.h" // todo: remove png.h

#if PLATFORM_MAC
#include <sys/stat.h>
#include <unistd.h>
#endif

#if WITH_EDITORONLY_DATA

// Disable warning "interaction between '_setjmp' and C++ object destruction is non-portable"
// and 'fopen': This function or variable may be unsafe.
#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable:4611)
#pragma warning(disable:4996)
#endif

bool FGoogleARCoreSessionConfigCookSupport::SaveTextureToPNG(UTexture2D *Tex, const FString &Filename)
{
	// todo: use FImageUtils::SaveTexture2D instead
	//  
	#if 0

	FImage Image;
	if ( ! FImageUtils::GetTexture2DSourceImage(Tex,Image) )
		return false;
	return FImageUtils::SaveImageByExtension(*FileName,Image);

	#endif

	TArray64<uint8> MipData;
	bool Ret = true;

	if (Tex->Source.GetMipData(MipData, 0))
	{
		if (Tex->Source.GetFormat() != TSF_BGRA8 )
		{
			UE_LOGF(
				LogGoogleARCore, Error,
				"Texture %ls is not RGBA8 or BGRA8 and cannot be used as a tracking target.",
				*Tex->GetName());

			return false;
		}

		int32 Width = Tex->Source.GetSizeX();
		int32 Height = Tex->Source.GetSizeY();

		png_structp PngPtr = png_create_write_struct(PNG_LIBPNG_VER_STRING, nullptr, nullptr, nullptr);

		if (PngPtr)
		{
			png_infop PngInfoPtr = png_create_info_struct(PngPtr);

			// We're using C file IO here just because of libPNG interop.
			FILE *OutputFile = fopen(TCHAR_TO_ANSI(*Filename), "wb+");

			if (setjmp(png_jmpbuf(PngPtr)))
			{
				UE_LOGF(
					LogGoogleARCore, Error,
					"Error writing PNG for texture %ls.",
					*Tex->GetName());
				Ret = false;
			}
			else
			{
				png_init_io(PngPtr, OutputFile);
				png_set_IHDR(
					PngPtr, PngInfoPtr, Width, Height,
					8, PNG_COLOR_TYPE_RGB, PNG_INTERLACE_NONE,
					PNG_COMPRESSION_TYPE_BASE, PNG_FILTER_TYPE_BASE);
				png_write_info(PngPtr, PngInfoPtr);

				uint8_t *Row = new uint8_t[Width * 3];

				for (int32 y = 0; y < Height; y++)
				{
					for (int32 x = 0; x < Width; x++)
					{
						for (int32 c = 0; c < 3; c++)
						{
							int32 RealChannel = c;
							if (Tex->Source.GetFormat() == TSF_BGRA8)
							{
								if (RealChannel == 0) {
									RealChannel = 2;
								}
								else if (RealChannel == 2) {
									RealChannel = 0;
								}
							}
							Row[x * 3 + c] = MipData[(y * Width + x) * 4 + RealChannel];
						}
					}
					png_write_row(PngPtr, Row);
				}

				png_write_end(PngPtr, NULL);

				delete[] Row;
			}

			if (OutputFile)
			{
				fclose(OutputFile);
			}

			if (PngInfoPtr)
			{
				png_free_data(PngPtr, PngInfoPtr, PNG_FREE_ALL, -1);
			}

			png_destroy_write_struct(&PngPtr, nullptr);
		}
	}
	else
	{
		UE_LOGF(
			LogGoogleARCore, Error,
			"Error reading mip data in texture %ls.",
			*Tex->GetName());

		return false;
	}

	return Ret;
}

#if PLATFORM_LINUX || PLATFORM_MAC
bool FGoogleARCoreSessionConfigCookSupport::PlatformSetExecutable(const TCHAR* Filename, bool bIsExecutable)
{
	struct stat FileInfo;
	FTCHARToUTF8 FilenameUtf8(Filename);
	bool bSuccess = false;
	// Set executable permission bit
	if (stat(FilenameUtf8.Get(), &FileInfo) == 0)
	{
		mode_t ExeFlags = S_IXUSR | S_IXGRP | S_IXOTH;
		FileInfo.st_mode = bIsExecutable ? (FileInfo.st_mode | ExeFlags) : (FileInfo.st_mode & ~ExeFlags);
		bSuccess = chmod(FilenameUtf8.Get(), FileInfo.st_mode) == 0;
	}
	return bSuccess;
}
#endif

// Renable warning "interaction between '_setjmp' and C++ object destruction is non-portable"
// and 'fopen': This function or variable may be unsafe.
#ifdef _MSC_VER
#pragma warning(pop)
#endif

void FGoogleARCoreSessionConfigCookSupport::RegisterModuleFeature()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
}

void FGoogleARCoreSessionConfigCookSupport::UnregisterModuleFeature()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
}

void FGoogleARCoreSessionConfigCookSupport::OnSerializeSessionConfig(UARSessionConfig* SessionConfig, FArchive& Ar, TArray<uint8>& SerializedARCandidateImageDatabase)
{
	if (!Ar.CookingTarget()->PlatformName().Contains("Android")) {
		return;
	}

	FString OutStdout;
	FString OutStderr;
	int32 OutReturnCode = 0;

	SerializedARCandidateImageDatabase.Empty();
	const TArray<UARCandidateImage*>& CandidateImageList = SessionConfig->GetCandidateImageList();

	if (CandidateImageList.Num()) {
		UE_LOGF(LogGoogleARCore, Display, "Cooking ARSessionConfig for platform: %ls", *Ar.CookingTarget()->PlatformName());
		FString PathToDbTool =
				FPaths::Combine(
					*FPaths::EnginePluginsDir(),
					TEXT("Runtime"),
					TEXT("AR"),
					TEXT("Google"),
					TEXT("GoogleARCore"),
					TEXT("Binaries"),
					TEXT("ThirdParty"),
					TEXT("Google"),
					TEXT("ARCoreImg"),
					*UGameplayStatics::GetPlatformName(),
#if PLATFORM_LINUX
					TEXT("arcoreimg")
#elif PLATFORM_WINDOWS
					TEXT("arcoreimg.exe")
#elif PLATFORM_MAC
					TEXT("ptdbtool_macos_lipobin")
#endif
			);

		FString TempDir =
			FPaths::ConvertRelativePathToFull(
				FPaths::Combine(
					*FPaths::EnginePluginsDir(),
					TEXT("Runtime"),
					TEXT("AR"),
					TEXT("Google"),
					TEXT("GoogleARCore"),
					TEXT("Intermediate"),
					TEXT("ARCoreImgTemp")));

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		if (!PlatformFile.DirectoryExists(*TempDir))
		{
			PlatformFile.CreateDirectory(*TempDir);
		}

		TArray<FString> CleanupList;

		FString ImageListFileContents;

		for (int32 i = 0; i < CandidateImageList.Num(); i++) 
		{
			if (!CandidateImageList[i])
			{
				continue;
			}
			
			UTexture2D *Tex = CandidateImageList[i]->GetCandidateTexture();

			if (Tex) {

				FString PNGFilename =
					FPaths::Combine(TempDir, Tex->GetName() + FString(".png"));

				if (SaveTextureToPNG(
					Tex,
					PNGFilename))
				{
					// "|" is used as a delimeter in the image
					// list, and there doesn't seem to be any way
					// to escape them, so they will be replaced
					// with underscores.
					FString TmpName =
						CandidateImageList[i]->GetFriendlyName().Replace(TEXT("|"), TEXT("_"));

					ImageListFileContents +=
						TmpName +
						FString("|") +
						PNGFilename;

					CleanupList.Add(PNGFilename);

					if (CandidateImageList[i]->GetPhysicalWidth() > 0.0f) {
						ImageListFileContents +=
							FString("|") +
							FString::SanitizeFloat(CandidateImageList[i]->GetPhysicalWidth() / 100.0f); // convert cm to meter.
					}

					ImageListFileContents += FString("\n");
				}

			}
		}

		FString PathToImageList =
			FPaths::Combine(TempDir, TEXT("image_list.txt"));
		FString PathToImageDb =
			FPaths::Combine(TempDir, TEXT("image_list.imgdb"));

		CleanupList.Add(PathToImageList);
		CleanupList.Add(PathToImageDb);

		FFileHelper::SaveStringToFile(
			ImageListFileContents,
			*PathToImageList);

		OutStderr = "";
		OutStdout = "";
		OutReturnCode = 0;
		FString DbToolParams = FString("build-db --input_image_list_path=\"") + PathToImageList +
			FString("\" --output_db_path=\"") + PathToImageDb + "\"";

#if PLATFORM_LINUX || PLATFORM_MAC
		PlatformSetExecutable(*PathToDbTool, true);
#endif

		FPlatformProcess::ExecProcess(
			*PathToDbTool,
			*DbToolParams,
			&OutReturnCode,
			&OutStdout,
			&OutStderr);

		if (OutReturnCode)
		{
			Ar.SetCriticalError();
			UE_LOGF(LogGoogleARCore, Error, "Failed to build augmented image database: %ls", *OutStderr);
		}
		else
		{
			FFileHelper::LoadFileToArray(SerializedARCandidateImageDatabase, *PathToImageDb, 0);
			UE_LOGF(LogGoogleARCore, Display,
				"Augmented image database created. Size: %d bytes. Tool output: %ls",
				SerializedARCandidateImageDatabase.Num(), *OutStdout);
		}

		for (int32 i = 0; i < CleanupList.Num(); i++)
		{
			UE_LOGF(LogGoogleARCore, Log, "Cleaning up: %ls", *CleanupList[i]);
			PlatformFile.DeleteFile(*CleanupList[i]);
		}

		PlatformFile.DeleteDirectory(*TempDir);
	}
}

#endif
