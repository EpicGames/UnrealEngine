// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITORONLY_DATA
#include "ARSessionConfigCookSupport.h"

class FGoogleARCoreSessionConfigCookSupport : public IARSessionConfigCookSupport
{
public:
	static bool SaveTextureToPNG(class UTexture2D *Tex, const FString &Filename);
#if PLATFORM_LINUX || PLATFORM_MAC
	static bool PlatformSetExecutable(const TCHAR* Filename, bool bIsExecutable);
#endif

	void RegisterModuleFeature();
	void UnregisterModuleFeature();

	virtual void OnSerializeSessionConfig(UARSessionConfig* SessionConfig, FArchive& Ar, TArray<uint8>& SerializedARCandidateImageDatabase) override;
};
#endif
