// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreDependencyHandler.h"
#include "GoogleARCoreDevice.h"
#include "LatentActions.h"
#include "GoogleARCoreBaseLogCategory.h"
#include "ARBlueprintLibrary.h"
#include "GoogleARCoreXRTrackingSystem.h"


static EARServiceAvailability ToARServiceAvailability(EGoogleARCoreAvailability ARCoreAvailability)
{
	switch (ARCoreAvailability)
	{
		case EGoogleARCoreAvailability::UnknownError:
			return EARServiceAvailability::UnknownError;
			
		case EGoogleARCoreAvailability::UnknownChecking:
			return EARServiceAvailability::UnknownChecking;
			
		case EGoogleARCoreAvailability::UnknownTimedOut:
			return EARServiceAvailability::UnknownTimedOut;
			
		case EGoogleARCoreAvailability::UnsupportedDeviceNotCapable:
			return EARServiceAvailability::UnsupportedDeviceNotCapable;
			
		case EGoogleARCoreAvailability::SupportedNotInstalled:
			return EARServiceAvailability::SupportedNotInstalled;
			
		case EGoogleARCoreAvailability::SupportedApkTooOld:
			return EARServiceAvailability::SupportedVersionTooOld;
			
		case EGoogleARCoreAvailability::SupportedInstalled:
			return EARServiceAvailability::SupportedInstalled;
	}
	
	return EARServiceAvailability::UnknownError;
}

static EARServiceInstallRequestResult ToARServiceInstallRequestResult(EGoogleARCoreAPIStatus RequestStatus)
{
	EARServiceInstallRequestResult OutRequestResult = EARServiceInstallRequestResult::FatalError;
	switch (RequestStatus)
	{
		case EGoogleARCoreAPIStatus::AR_SUCCESS:
			OutRequestResult = EARServiceInstallRequestResult::Installed;
			break;
		case EGoogleARCoreAPIStatus::AR_ERROR_FATAL:
			OutRequestResult = EARServiceInstallRequestResult::FatalError;
			break;
		case EGoogleARCoreAPIStatus::AR_UNAVAILABLE_DEVICE_NOT_COMPATIBLE:
			OutRequestResult = EARServiceInstallRequestResult::DeviceNotCompatible;
			break;
		case EGoogleARCoreAPIStatus::AR_UNAVAILABLE_USER_DECLINED_INSTALLATION:
			OutRequestResult = EARServiceInstallRequestResult::UserDeclinedInstallation;
			break;
		default:
			ensureMsgf(false, TEXT("Unexpected ARCore API Status: %d"), static_cast<int>(RequestStatus));
			break;
	}
	return OutRequestResult;
}

struct FBaseLatentAction : public FPendingLatentAction
{
public:
	FBaseLatentAction(const FLatentActionInfo& InLatentInfo, const FString& InDescription)
	: FPendingLatentAction()
	, ExecutionFunction(InLatentInfo.ExecutionFunction)
	, OutputLink(InLatentInfo.Linkage)
	, CallbackTarget(InLatentInfo.CallbackTarget)
	, Description(InDescription)
	{}
	
#if WITH_EDITOR
	FString GetDescription() const override
	{
		return Description;
	}
#endif
	
protected:
	void FinishAction(FLatentResponse& Response)
	{
		Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
	}
	
protected:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;
	FString Description;
};

struct FCheckARCoreAvailabilityAction : public FBaseLatentAction
{
public:
	FCheckARCoreAvailabilityAction(const FLatentActionInfo& InLatentInfo, EARServiceAvailability& InAvailability)
	: FBaseLatentAction(InLatentInfo, TEXT("Checking ARCore availability."))
	, OutAvailability(InAvailability)
	{}
	
	void UpdateOperation(FLatentResponse& Response) override
	{
		auto ARCoreAvailability = FGoogleARCoreDevice::GetInstance()->CheckARCoreAPKAvailability();
		if (ARCoreAvailability != EGoogleARCoreAvailability::UnknownChecking)
		{
			OutAvailability = ToARServiceAvailability(ARCoreAvailability);
			FinishAction(Response);
		}
	}
	
private:
	EARServiceAvailability& OutAvailability;
};

struct FInstallARCoreAction : public FBaseLatentAction
{
public:
	FInstallARCoreAction(const FLatentActionInfo& InLatentInfo, EARServiceInstallRequestResult& InRequestResult)
	: FBaseLatentAction(InLatentInfo, TEXT("Installing ARCore."))
	, OutRequestResult(InRequestResult)
	{}

	void UpdateOperation(FLatentResponse& Response) override
	{
		EGoogleARCoreInstallStatus InstallStatus = EGoogleARCoreInstallStatus::Installed;
		EGoogleARCoreAPIStatus RequestStatus = FGoogleARCoreDevice::GetInstance()->RequestInstall(!bInstallRequested, InstallStatus);
		if (RequestStatus != EGoogleARCoreAPIStatus::AR_SUCCESS)
		{
			OutRequestResult = ToARServiceInstallRequestResult(RequestStatus);
			FinishAction(Response);
		}
		else if (InstallStatus == EGoogleARCoreInstallStatus::Installed)
		{
			OutRequestResult = EARServiceInstallRequestResult::Installed;
			FinishAction(Response);
		}
		else
		{
			// InstallSttatus returns requested.
			bInstallRequested = true;
		}
	}
	
private:
	EARServiceInstallRequestResult& OutRequestResult;
	bool bInstallRequested = false;
};

struct FRequestARCorePermissionAction : public FBaseLatentAction
{
public:
	FRequestARCorePermissionAction(const FLatentActionInfo& InLatentInfo, EARServicePermissionRequestResult& InResult)
	: FBaseLatentAction(InLatentInfo, TEXT("Requesting ARCore permissions"))
	, OutResult(InResult)
	{}
	
	void UpdateOperation(FLatentResponse& Response) override
	{
		if (SessionConfig.IsValid())
		{
			const auto PermissionStatus = FGoogleARCoreDevice::GetInstance()->CheckAndRequrestPermission(*SessionConfig.Get());
			if (PermissionStatus == EARCorePermissionStatus::Granted)
			{
				OutResult = EARServicePermissionRequestResult::Granted;
				FinishAction(Response);
			}
			else if (PermissionStatus == EARCorePermissionStatus::Denied)
			{
				OutResult = EARServicePermissionRequestResult::Denied;
				FinishAction(Response);
			}
		}
	}
	
	void SetSessionConfig(UARSessionConfig* InSessionConfig)
	{
		check(InSessionConfig);
		SessionConfig = InSessionConfig;
	}
	
private:
	EARServicePermissionRequestResult& OutResult;
	TWeakObjectPtr<UARSessionConfig> SessionConfig;
};

struct FStartARSessionAction : public FBaseLatentAction
{
public:
	FStartARSessionAction(const FLatentActionInfo& InLatentInfo, int& InResult)
	: FBaseLatentAction(InLatentInfo, TEXT("Starting AR session"))
	{}
	
	void UpdateOperation(FLatentResponse& Response) override
	{
		if (FGoogleARCoreDevice::GetInstance()->GetStartSessionRequestFinished())
		{
			FinishAction(Response);
		}
	}
};

template<class TAction, class TResult>
static TAction* AddLatentAction(UObject* WorldContextObject, FLatentActionInfo LatentInfo, TResult& OutResult)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		if (LatentManager.FindExistingAction<TAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			auto NewAction = new TAction(LatentInfo, OutResult);
			LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
			return NewAction;
		}
	}
	
	return nullptr;
}

void UGoogleARCoreDependencyHandler::StartARSessionLatent(UObject* WorldContextObject, UARSessionConfig* SessionConfig, FLatentActionInfo LatentInfo)
{
	if (!SessionConfig)
	{
		UE_LOGF(LogGoogleARCore, Error, "StartARSessionLatent requires a valid UARSessionConfig object!");
		return;
	}
	
	auto DummyResult = 0;
	if (auto Action = AddLatentAction<FStartARSessionAction>(WorldContextObject, LatentInfo, DummyResult))
	{
		UARBlueprintLibrary::StartARSession(SessionConfig);
	}
}

void UGoogleARCoreDependencyHandler::CheckARServiceAvailability(UObject* WorldContextObject, FLatentActionInfo LatentInfo, EARServiceAvailability& OutAvailability)
{
	AddLatentAction<FCheckARCoreAvailabilityAction>(WorldContextObject, LatentInfo, OutAvailability);
}

void UGoogleARCoreDependencyHandler::InstallARService(UObject* WorldContextObject, FLatentActionInfo LatentInfo, EARServiceInstallRequestResult& OutInstallResult)
{
	AddLatentAction<FInstallARCoreAction>(WorldContextObject, LatentInfo, OutInstallResult);
}

void UGoogleARCoreDependencyHandler::RequestARSessionPermission(UObject* WorldContextObject, UARSessionConfig* SessionConfig, FLatentActionInfo LatentInfo, EARServicePermissionRequestResult& OutPermissionResult)
{
	if (!SessionConfig)
	{
		UE_LOGF(LogGoogleARCore, Error, "RequestARSessionPermission requires a valid UARSessionConfig object!");
		return;
	}
	
	if (auto Action = AddLatentAction<FRequestARCorePermissionAction>(WorldContextObject, LatentInfo, OutPermissionResult))
	{
		Action->SetSessionConfig(SessionConfig);
	}
}
