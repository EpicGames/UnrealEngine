// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARDependencyHandler.h"

#include "GoogleARCoreDependencyHandler.generated.h"


UCLASS()
class GOOGLEARCOREBASE_API UGoogleARCoreDependencyHandler : public UARDependencyHandler
{
	GENERATED_BODY()
	
public:
	void StartARSessionLatent(UObject* WorldContextObject, UARSessionConfig* SessionConfig, FLatentActionInfo LatentInfo) override;
	void CheckARServiceAvailability(UObject* WorldContextObject, FLatentActionInfo LatentInfo, EARServiceAvailability& OutAvailability) override;
	void InstallARService(UObject* WorldContextObject, FLatentActionInfo LatentInfo, EARServiceInstallRequestResult& OutInstallResult) override;
	void RequestARSessionPermission(UObject* WorldContextObject, UARSessionConfig* SessionConfig, FLatentActionInfo LatentInfo, EARServicePermissionRequestResult& OutPermissionResult) override;
};
