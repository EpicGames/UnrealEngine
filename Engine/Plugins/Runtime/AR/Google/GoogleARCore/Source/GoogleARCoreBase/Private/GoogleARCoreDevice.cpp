// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreDevice.h"
#include "CoreMinimal.h"
#include "Misc/ScopeLock.h"
#include "HAL/ThreadSafeCounter64.h"
#include "GameFramework/WorldSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "GeneralProjectSettings.h"
#include "GoogleARCoreXRTrackingSystem.h"
#include "GoogleARCoreAndroidHelper.h"
#include "GoogleARCoreBaseLogCategory.h"
#include "GoogleARCoreTexture.h"

#if PLATFORM_ANDROID
#include "Android/AndroidApplication.h"
#endif

#include "GoogleARCorePermissionHandler.h"

DECLARE_CYCLE_STAT(TEXT("UpdateGameFrame"), STAT_UpdateGameFrame, STATGROUP_ARCore);

#define NUM_CAMERA_TEXTURES 4

namespace
{
	EGoogleARCoreFunctionStatus ToARCoreFunctionStatus(EGoogleARCoreAPIStatus Status)
	{
		switch (Status)
		{
		case EGoogleARCoreAPIStatus::AR_SUCCESS:
			return EGoogleARCoreFunctionStatus::Success;
		case EGoogleARCoreAPIStatus::AR_ERROR_NOT_TRACKING:
			return EGoogleARCoreFunctionStatus::NotTracking;
		case EGoogleARCoreAPIStatus::AR_ERROR_SESSION_PAUSED:
			return EGoogleARCoreFunctionStatus::SessionPaused;
		case EGoogleARCoreAPIStatus::AR_ERROR_RESOURCE_EXHAUSTED:
			return EGoogleARCoreFunctionStatus::ResourceExhausted;
		case EGoogleARCoreAPIStatus::AR_ERROR_NOT_YET_AVAILABLE:
			return EGoogleARCoreFunctionStatus::NotAvailable;
		case EGoogleARCoreAPIStatus::AR_ERROR_ILLEGAL_STATE:
			return EGoogleARCoreFunctionStatus::IllegalState;
		default:
			ensureMsgf(false, TEXT("Unknown conversion from EGoogleARCoreAPIStatus %d to EGoogleARCoreFunctionStatus."), static_cast<int>(Status));
			return EGoogleARCoreFunctionStatus::Unknown;
		}
	}
}

static bool ShouldUseVulkan()
{
#if PLATFORM_ANDROID
	return FAndroidMisc::ShouldUseVulkan();
#else
	return false;
#endif
}

FGoogleARCoreDelegates::FGoogleARCoreOnConfigCameraDelegate FGoogleARCoreDelegates::OnCameraConfig;

FGoogleARCoreDevice* FGoogleARCoreDevice::GetInstance()
{
	static FGoogleARCoreDevice Inst;
	return &Inst;
}

FGoogleARCoreDevice::FGoogleARCoreDevice()
	: bIsARCoreSessionRunning(false)
	, bStartSessionRequested(false)
	, bShouldSessionRestart(false)
	, bARCoreInstallRequested(false)
	, bARCoreInstalled(false)
	, WorldToMeterScale(100.0f)
	, PermissionHandler(nullptr)
	, bDisplayOrientationChanged(false)
	, CurrentSessionStatus(EARSessionStatus::NotStarted, TEXT("ARCore Session is uninitialized."))
{
}

EGoogleARCoreAvailability FGoogleARCoreDevice::CheckARCoreAPKAvailability()
{
	return FGoogleARCoreAPKManager::CheckARCoreAPKAvailability();
}

EGoogleARCoreAPIStatus FGoogleARCoreDevice::RequestInstall(bool bUserRequestedInstall, EGoogleARCoreInstallStatus& OutInstallStatus)
{
	return FGoogleARCoreAPKManager::RequestInstall(bUserRequestedInstall, OutInstallStatus);
}

bool FGoogleARCoreDevice::GetIsTrackingTypeSupported(EARSessionType SessionType)
{
	if (SessionType == EARSessionType::World || SessionType == EARSessionType::Face)
	{
		return true;
	}
	else
	{
		return false;
	}
}

void FGoogleARCoreDevice::OnModuleLoaded()
{
	// Init display orientation.
	OnDisplayOrientationChanged();
}

void FGoogleARCoreDevice::OnModuleUnloaded()
{
	// clear the shared ptr.
	ARCoreSession.Reset();
}

bool FGoogleARCoreDevice::GetIsARCoreSessionRunning()
{
	return bIsARCoreSessionRunning;
}

FARSessionStatus FGoogleARCoreDevice::GetSessionStatus()
{
	return CurrentSessionStatus;
}

float FGoogleARCoreDevice::GetWorldToMetersScale()
{
	return WorldToMeterScale;
}

// This function will be called by public function to start AR core session request.
void FGoogleARCoreDevice::StartARCoreSessionRequest(UARSessionConfig* SessionConfig)
{
	UE_LOGF(LogGoogleARCore, Log, "Start ARCore session requested.");
	
	if (!GCObject)
	{
		GCObject = MakeShared<FInternalGCObject, ESPMode::ThreadSafe>(this);
	}
	
#if PLATFORM_ANDROID
	UE_LOGF(LogGoogleARCore, Display, "ARCore session using %ls.", ShouldUseVulkan() ? TEXT("Vulkan") : TEXT("OpenGL ES"));
#endif

	if (bIsARCoreSessionRunning)
	{
		UE_LOGF(LogGoogleARCore, Log, "ARCore session is already running, set it to use the new session config.");
		EGoogleARCoreAPIStatus Status = ARCoreSession->ConfigSession(*SessionConfig);
		ensureMsgf(Status == EGoogleARCoreAPIStatus::AR_SUCCESS, TEXT("Failed to set ARCore session to new configuration while it is running."));

		return;
	}

	if (bStartSessionRequested)
	{
		UE_LOGF(LogGoogleARCore, Warning, "ARCore session is already starting. This will overriding the previous session config with the new one.")
	}
	else 
	{
		PauseARCoreSession();
	}

	bStartSessionRequested = true;
	bARCoreInstallRequested = false;

	// Try recreating the ARCoreSession to fix the fatal error.
	if (CurrentSessionStatus.Status == EARSessionStatus::FatalError)
	{
		UE_LOGF(LogGoogleARCore, Warning, "Reset ARCore session due to fatal error detected.");
		ResetARCoreSession();
	}
}

bool FGoogleARCoreDevice::SetARCameraConfig(FGoogleARCoreCameraConfig CameraConfig)
{
	EGoogleARCoreAPIStatus APIStatus = ARCoreSession->SetCameraConfig(CameraConfig);
	if (APIStatus == EGoogleARCoreAPIStatus::AR_SUCCESS)
	{
		return true;
	}
	else if (APIStatus == EGoogleARCoreAPIStatus::AR_ERROR_SESSION_NOT_PAUSED)
	{
		UE_LOGF(LogGoogleARCore, Error, "Failed to set ARCamera configuration due to the arcore session isn't paused.");
	}
	else
	{
		UE_LOGF(LogGoogleARCore, Error, "Failed to set ARCamera configuration with provided CameraConfig.");
	}

	UE_LOGF(LogGoogleARCore, Error, "You should only call the ConfigARCoreCamera function when the OnConfigCamera delegate get called, and the provided CameraConfig must be from the array that is passed by the delegate.");
	return false;
}

bool FGoogleARCoreDevice::GetARCameraConfig(FGoogleARCoreCameraConfig& OutCurrentCameraConfig)
{
	if (ARCoreSession.IsValid())
	{
		ARCoreSession->GetARCameraConfig(OutCurrentCameraConfig);
		return true;
	}
	else
	{
		return false;
	}
}

bool FGoogleARCoreDevice::GetIsFrontCameraSession()
{
	UARSessionConfig* CurrentConfig = AccessSessionConfig();

	if (CurrentConfig->GetSessionType() == EARSessionType::Face)
	{
		return true;
	}

	UGoogleARCoreSessionConfig* ARCoreSessionConfig = Cast<UGoogleARCoreSessionConfig>(CurrentConfig);
	if (ARCoreSessionConfig != nullptr && ARCoreSessionConfig->CameraFacing == EGoogleARCoreCameraFacing::Front)
	{
		return true;
	}

	return false;
}

bool FGoogleARCoreDevice::GetShouldInvertCulling()
{
	UARSessionConfig* CurrentConfig = AccessSessionConfig();
	UGoogleARCoreSessionConfig* ARCoreSessionConfig = Cast<UGoogleARCoreSessionConfig>(CurrentConfig);

	if(!CurrentConfig->ShouldEnableCameraTracking())
	{
		return false;
	}

	if (CurrentConfig->GetSessionType() == EARSessionType::Face)
	{
		return true;
	}

	if (ARCoreSessionConfig != nullptr && ARCoreSessionConfig->CameraFacing == EGoogleARCoreCameraFacing::Front)
	{
		return true;
	}

	return false;
}

int FGoogleARCoreDevice::AddRuntimeAugmentedImage(UGoogleARCoreAugmentedImageDatabase* TargetImageDatabase, const TArray<uint8>& ImageGrayscalePixels,
	int ImageWidth, int ImageHeight, FString ImageName, float ImageWidthInMeter)
{
	if (!ARCoreSession.IsValid())
	{
		UE_LOGF(LogGoogleARCore, Warning, "Failed to add runtime augmented image: No valid session!");
		return -1;
	}

	return ARCoreSession->AddRuntimeAugmentedImage(TargetImageDatabase, ImageGrayscalePixels, ImageWidth, ImageHeight, ImageName, ImageWidthInMeter);
}

bool FGoogleARCoreDevice::AddRuntimeCandidateImage(UARSessionConfig* SessionConfig, const TArray<uint8>& ImageGrayscalePixels, int ImageWidth, int ImageHeight,
	FString FriendlyName, float PhysicalWidth)
{
	if (!ARCoreSession.IsValid())
	{
		UE_LOGF(LogGoogleARCore, Warning, "Failed to add runtime candidate image: No valid session!");
		return false;
	}

	return ARCoreSession->AddRuntimeCandidateImage(SessionConfig, ImageGrayscalePixels, ImageWidth, ImageHeight, FriendlyName, PhysicalWidth);
}

bool FGoogleARCoreDevice::GetStartSessionRequestFinished()
{
	return !bStartSessionRequested;
}

// Note that this function will only be registered when ARCore is supported.
void FGoogleARCoreDevice::UpdateGameFrame(UWorld* World)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateGameFrame);
	
	WorldToMeterScale = World->GetWorldSettings()->WorldToMeters;
	TFunction<void()> Func;
	while (RunOnGameThreadQueue.Dequeue(Func))
	{
		Func();
	}

	if (!bIsARCoreSessionRunning && bStartSessionRequested)
	{
		if (!bARCoreInstalled)
		{
			EGoogleARCoreInstallStatus InstallStatus = EGoogleARCoreInstallStatus::Installed;
			EGoogleARCoreAPIStatus Status = FGoogleARCoreAPKManager::RequestInstall(!bARCoreInstallRequested, InstallStatus);

			if (Status != EGoogleARCoreAPIStatus::AR_SUCCESS)
			{
				bStartSessionRequested = false;
				CurrentSessionStatus.Status = EARSessionStatus::NotSupported;
				CurrentSessionStatus.AdditionalInfo = TEXT("ARCore APK installation failed on this device.");
			}
			else if (InstallStatus == EGoogleARCoreInstallStatus::Installed)
			{
				bARCoreInstalled = true;
			}
			else
			{
				bARCoreInstallRequested = true;
			}
		}
		else if (PermissionStatus == EARCorePermissionStatus::Denied)
		{
			CurrentSessionStatus.Status = EARSessionStatus::PermissionNotGranted;
			CurrentSessionStatus.AdditionalInfo = TEXT("Camera permission has been denied by the user.");
			bStartSessionRequested = false;
		}
		else
		{
			CheckAndRequrestPermission(*AccessSessionConfig());
			// Either we don't need to request permission or the permission request is done.
			// Queue the session start task on UiThread
			if (PermissionStatus == EARCorePermissionStatus::Granted)
			{
				StartSessionWithRequestedConfig();
			}
		}
	}

	if (bIsARCoreSessionRunning)
	{
		// Update ARFrame
		FVector2D ViewportSize(1, 1);
		if (GEngine && GEngine->GameViewport)
		{
			ViewportSize = GEngine->GameViewport->Viewport->GetSizeXY();
		}
		ARCoreSession->SetDisplayGeometry(static_cast<int>(FGoogleARCoreAndroidHelper::GetDisplayRotation()), ViewportSize.X, ViewportSize.Y);
		EGoogleARCoreAPIStatus Status = ARCoreSession->Update(WorldToMeterScale);
		if (Status == EGoogleARCoreAPIStatus::AR_ERROR_FATAL)
		{
			ARCoreSession->Pause();
			bIsARCoreSessionRunning = false;
			CurrentSessionStatus.Status = EARSessionStatus::FatalError;
			CurrentSessionStatus.AdditionalInfo = TEXT("Fatal error occurred when updating ARCore Session. Stopping and restarting ARCore Session may fix the issue.");
		}
		else if (ARCoreSession->HasNewFrame()) // Only update textures if the session has a new frame
		{
			if (auto Frame = ARCoreSession->GetLatestFrame())
			{
				// With Vulkan update the camera texture with the current frame's hardware buffer.
				if (ShouldUseVulkan())
				{
					const uint32 CurrentFrame = ARCoreSession->GetFrameNum() % PassthroughCameraTextures.Num();
					const uint32 CurrentCameraTextureId = PassthroughCameraTextureKeys[CurrentFrame];

					if (Frame->UpdateCameraTexture(PassthroughCameraTextures[CurrentCameraTextureId]))
					{
						LastCameraTextureId = CurrentCameraTextureId;
					}
				}
				// With OpenGL just use the texture id returned by the frame.
				else
				{
					LastCameraTextureId = Frame->GetCameraTextureId();
				}

				if (ARCoreSession->IsSceneDepthEnabled())
				{
					Frame->UpdateDepthTexture(MutableView(DepthTexture));
				}
			}
		}
	}
}

EARCorePermissionStatus FGoogleARCoreDevice::CheckAndRequrestPermission(const UARSessionConfig& ConfigurationData)
{
	if (PermissionStatus == EARCorePermissionStatus::Unknown)
	{
		TArray<FString> RuntimePermissions;
		TArray<FString> NeededPermissions;
		GetRequiredRuntimePermissionsForConfiguration(ConfigurationData, RuntimePermissions);
		if (RuntimePermissions.Num() > 0)
		{
			for (int32 i = 0; i < RuntimePermissions.Num(); i++)
			{
				if (!UARCoreAndroidPermissionHandler::CheckRuntimePermission(RuntimePermissions[i]))
				{
					NeededPermissions.Add(RuntimePermissions[i]);
				}
			}
		}
		
		if (NeededPermissions.Num() > 0)
		{
			PermissionStatus = EARCorePermissionStatus::Requested;
			if (PermissionHandler == nullptr)
			{
				PermissionHandler = NewObject<UARCoreAndroidPermissionHandler>();
			}
			PermissionHandler->RequestRuntimePermissions(NeededPermissions);
		}
		else
		{
			PermissionStatus = EARCorePermissionStatus::Granted;
		}
	}
	
	return PermissionStatus;
}

void FGoogleARCoreDevice::HandleRuntimePermissionsGranted(const TArray<FString>& RuntimePermissions, const TArray<bool>& Granted)
{
	bool bGranted = true;
	for (int32 i = 0; i < RuntimePermissions.Num(); i++)
	{
		if (!Granted[i])
		{
			bGranted = false;
			UE_LOGF(LogGoogleARCore, Warning, "Android runtime permission denied: %ls", *RuntimePermissions[i]);
		}
		else
		{
			UE_LOGF(LogGoogleARCore, Log, "Android runtime permission granted: %ls", *RuntimePermissions[i]);
		}
	}
	
	PermissionStatus = bGranted ? EARCorePermissionStatus::Granted : EARCorePermissionStatus::Denied;
}

void FGoogleARCoreDevice::StartSessionWithRequestedConfig()
{
	bStartSessionRequested = false;

	ARCoreSession = CreateSession();

	if (!ARCoreSession.IsValid())
	{
		return;
	}

	StartSession();
}

TSharedPtr<FGoogleARCoreSession> FGoogleARCoreDevice::CreateSession()
{
	TSharedPtr<FGoogleARCoreSession> NewARCoreSession = FGoogleARCoreSession::CreateARCoreSession();
	EGoogleARCoreAPIStatus SessionCreateStatus = NewARCoreSession->GetSessionCreateStatus();
	if (SessionCreateStatus != EGoogleARCoreAPIStatus::AR_SUCCESS)
	{
		ensureMsgf(false, TEXT("Failed to create ARCore session with error status: %d"), (int)SessionCreateStatus);
		CurrentSessionStatus.AdditionalInfo =
			FString::Printf(TEXT("Failed to create ARCore session with error status: %d"), (int)SessionCreateStatus);

		if (SessionCreateStatus != EGoogleARCoreAPIStatus::AR_ERROR_FATAL)
		{
			CurrentSessionStatus.Status = EARSessionStatus::NotSupported;
		}
		else
		{
			CurrentSessionStatus.Status = EARSessionStatus::FatalError;
		}

		NewARCoreSession.Reset();
	}
	else
	{
		NewARCoreSession->SetARSystem(ARSystem.ToSharedRef());
	}

	return NewARCoreSession;
}


void FGoogleARCoreDevice::StartSession()
{
	UARSessionConfig* RequestedConfig = AccessSessionConfig();

	if (!GetIsTrackingTypeSupported(RequestedConfig->GetSessionType()))
	{
		UE_LOGF(LogGoogleARCore, Warning, "Start AR failed: Unsupported AR tracking type %d for GoogleARCore", static_cast<int>(RequestedConfig->GetSessionType()));
		CurrentSessionStatus.AdditionalInfo = TEXT("Unsupported AR tracking type. Only EARSessionType::World is supported by ARCore.");
		CurrentSessionStatus.Status = EARSessionStatus::UnsupportedConfiguration;
		return;
	}

	// Configure the camera.  This must happen before we configure the session.  AugmentedFaces, for example, require that the front facing camera be selected before it is configured.
	UE_LOGF(LogGoogleARCore, Log, "FGoogleARCoreDevice::StartSession() Configuring camera...");
	{
		EGoogleARCoreCameraFacing CameraFacing = EGoogleARCoreCameraFacing::Back;
		const UGoogleARCoreSessionConfig *GoogleConfig = Cast<UGoogleARCoreSessionConfig>(RequestedConfig);
		if (GoogleConfig != nullptr)
		{
			CameraFacing = GoogleConfig->CameraFacing;

			if (RequestedConfig->GetSessionType() == EARSessionType::Face && CameraFacing != EGoogleARCoreCameraFacing::Front)
			{
				UE_LOGF(LogGoogleARCore, Error, "EARSessionType::Face requested, but the rear camera has been explicitly specified in the session config.  This is not supported and the session create will fail.");
			}
		}
		else
		{
			// Face sessions on arcore require the front camera.
			if (RequestedConfig->GetSessionType() == EARSessionType::Face)
			{
				CameraFacing = EGoogleARCoreCameraFacing::Front;
			}
		}

		TArray<FGoogleARCoreCameraConfig> SupportedCameraConfigs = ARCoreSession->GetSupportedCameraConfig(CameraFacing);
		FGoogleARCoreDelegates::OnCameraConfig.Broadcast(SupportedCameraConfigs);
	
		UE_LOGF(LogGoogleARCore, Log, "Got %d supported camera config from ARCore:", SupportedCameraConfigs.Num());
		for (int Index = 0; Index < SupportedCameraConfigs.Num(); ++Index)
		{
			UE_LOGF(LogGoogleARCore, Log, "%d: %ls", Index + 1, *SupportedCameraConfigs[Index].ToLogString());
		}

		// If the session config specifies a meaningful video format, we'll try to match it here
		const FARVideoFormat DesiredVideoFormat = RequestedConfig->GetDesiredVideoFormat();
		const bool bHasDesiredVideoFormatResolution = DesiredVideoFormat.Width > 0 && DesiredVideoFormat.Height > 0;
		const bool bHasValidDesiredVideoFormatFPS = DesiredVideoFormat.FPS == 30 || DesiredVideoFormat.FPS == 60;
		const FGoogleARCoreCameraConfig* MatchCameraConfig = nullptr;
		if (bHasDesiredVideoFormatResolution)
		{
			if (bHasValidDesiredVideoFormatFPS)
			{
				for (const FGoogleARCoreCameraConfig& CameraConfig : SupportedCameraConfigs)
				{
					if (CameraConfig.CameraTextureResolution.X == DesiredVideoFormat.Width &&
						CameraConfig.CameraTextureResolution.Y == DesiredVideoFormat.Height &&
						CameraConfig.GetMaxFPS() == DesiredVideoFormat.FPS)
					{
						MatchCameraConfig = &CameraConfig;
						break;
					}
				}

				if (MatchCameraConfig)
				{
					SetARCameraConfig(*MatchCameraConfig);
					UE_LOGF(LogGoogleARCore, Log, "Found and applied camera config matching video format (%d x %d, %d FPS): %ls",
						DesiredVideoFormat.Width, DesiredVideoFormat.Height, DesiredVideoFormat.FPS, *MatchCameraConfig->ToLogString());
				}
				else
				{
					UE_LOGF(LogGoogleARCore, Warning, "Couldn't find any camera config matching desired video format (%d x %d, %d FPS)",
						DesiredVideoFormat.Width, DesiredVideoFormat.Height, DesiredVideoFormat.FPS);
				}
			}
			else
			{
				UE_LOGF(LogGoogleARCore, Warning, "ARCore camera only supports 30/60 FPS.  Default format will be used.  Check your ARSessionConfig.");
			}
		}
		else
		{
			UE_LOGF(LogGoogleARCore, Log, "DesiredVideoFormat width,height = %i,%i default format will be used.", DesiredVideoFormat.Width > 0, DesiredVideoFormat.Height);
		}
		
		// If we are using the front camera we must choose a config beacause the default will be a back camera config.
		// Here we did not have any settings to use to pick so we will just pick the first.
		const bool bUseFrontCamera = CameraFacing == EGoogleARCoreCameraFacing::Front;
		if (bUseFrontCamera && !MatchCameraConfig)
		{
			if (SupportedCameraConfigs.Num() == 0)
			{
				UE_LOGF(LogGoogleARCore, Error, "We want to use a front camera but did not find any front camera configs.  This device may not have a front camera.  The session may use the back camera or session create may fail.");
			}
			else
			{
				MatchCameraConfig = &(SupportedCameraConfigs[0]);
				SetARCameraConfig(*MatchCameraConfig);
				UE_LOGF(LogGoogleARCore, Log, "Found and applied a front camera config (%d x %d, %d FPS): %ls",
					DesiredVideoFormat.Width, DesiredVideoFormat.Height, DesiredVideoFormat.FPS, *MatchCameraConfig->ToLogString());
			}
		}
	}


	UE_LOGF(LogGoogleARCore, Log, "FGoogleARCoreDevice::StartSession() Configuring session...");
	EGoogleARCoreAPIStatus Status = ARCoreSession->ConfigSession(*RequestedConfig);

	if (Status != EGoogleARCoreAPIStatus::AR_SUCCESS)
	{
		UE_LOGF(LogGoogleARCore, Error, "ARCore Session start failed ConfigSession with error status %d", static_cast<int>(Status));
		CurrentSessionStatus.AdditionalInfo = TEXT("ARCore Session start failed due to unsupported ARSessionConfig.");
		CurrentSessionStatus.Status = EARSessionStatus::UnsupportedConfiguration;
		return;
	}


	Status = ARCoreSession->Resume();

	// Allocate passthrough camera texture if necessary.
	if (!PassthroughCameraTextures.Num())
	{
		AllocatePassthroughCameraTextures();
	}

	check(PassthroughCameraTextures.Num());
	TArray<uint32> TextureIds;
	PassthroughCameraTextures.GetKeys(TextureIds);
	UE_LOGF(LogGoogleARCore, Display, "SetCameraTextureIds %i textures", TextureIds.Num());
	ARCoreSession->SetCameraTextureIds(TextureIds);


	if (Status != EGoogleARCoreAPIStatus::AR_SUCCESS)
	{
		UE_LOGF(LogGoogleARCore, Error, "ARCore Session start failed Resume with error status %d", static_cast<int>(Status));

		if (Status == EGoogleARCoreAPIStatus::AR_ERROR_ILLEGAL_STATE)
		{
			CurrentSessionStatus.AdditionalInfo = TEXT("Failed to start ARCore Session due to illegal state: All camera images previously acquired must be released before resuming the session with a different camera configuration.");
			CurrentSessionStatus.Status = EARSessionStatus::Other;
		}
		else
		{
			// If we failed here, the only reason would be fatal error.
			CurrentSessionStatus.AdditionalInfo = TEXT("Fatal error occurred when starting ARCore Session. Stopping and restarting ARCore Session may fix the issue.");
			CurrentSessionStatus.Status = EARSessionStatus::FatalError;
		}

		return;
	}

	if (auto TrackingSystem = FGoogleARCoreXRTrackingSystem::GetInstance())
	{
		const bool bMatchFOV = RequestedConfig->ShouldRenderCameraOverlay();
		TrackingSystem->ConfigARCoreXRCamera(bMatchFOV, RequestedConfig->ShouldRenderCameraOverlay());
	}
	else
	{
		UE_LOGF(LogGoogleARCore, Error, "ERROR: GoogleARCoreXRTrackingSystem is not available.");
	}
	
	ARCoreSession->GetARCameraConfig(SessionCameraConfig);

	bIsARCoreSessionRunning = true;
	CurrentSessionStatus.Status = EARSessionStatus::Running;
	CurrentSessionStatus.AdditionalInfo = TEXT("ARCore Session is running.");
	UE_LOGF(LogGoogleARCore, Log, "ARCore session started successfully.");

	ARSystem->OnARSessionStarted.Broadcast();
}

void FGoogleARCoreDevice::SetARSystem(TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe> InARSystem)
{
	check(InARSystem.IsValid());
	ARSystem = InARSystem;
}

void* FGoogleARCoreDevice::GetARSessionRawPointer()
{
#if PLATFORM_ANDROID
	return reinterpret_cast<void*>(ARCoreSession->GetHandle());
#else
	return nullptr;
#endif
}

void* FGoogleARCoreDevice::GetGameThreadARFrameRawPointer()
{
#if PLATFORM_ANDROID
	return reinterpret_cast<void*>(ARCoreSession->GetLatestFrameRawPointer());
#else
	return nullptr;
#endif
}

TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe> FGoogleARCoreDevice::GetARSystem()
{
	return ARSystem;
}

void FGoogleARCoreDevice::PauseARCoreSession()
{
	UE_LOGF(LogGoogleARCore, Log, "Pausing ARCore session.");
	if (!bIsARCoreSessionRunning)
	{
		if(bStartSessionRequested)
		{
			bStartSessionRequested = false;
		}
		else
		{
			UE_LOGF(LogGoogleARCore, Log, "Could not stop ARCore tracking session because there is no running tracking session!");
		}
		return;
	}

	EGoogleARCoreAPIStatus Status = ARCoreSession->Pause();

	if (Status == EGoogleARCoreAPIStatus::AR_ERROR_FATAL)
	{
		CurrentSessionStatus.Status = EARSessionStatus::FatalError;
		CurrentSessionStatus.AdditionalInfo = TEXT("Fatal error occurred when starting ARCore Session. Stopping and restarting ARCore Session may fix the issue.");
	}
	else
	{
		CurrentSessionStatus.Status = EARSessionStatus::NotStarted;
		CurrentSessionStatus.AdditionalInfo = TEXT("ARCore Session is paused.");
	}
	bIsARCoreSessionRunning = false;
	UE_LOGF(LogGoogleARCore, Log, "ARCore session paused");
}

void FGoogleARCoreDevice::ResetARCoreSession()
{
	ARCoreSession.Reset();
	CurrentSessionStatus.Status = EARSessionStatus::NotStarted;
	CurrentSessionStatus.AdditionalInfo = TEXT("ARCore Session is uninitialized.");
	
	PassthroughCameraTextures = {};
	PassthroughCameraTextureKeys = {};
	LastCameraTextureId = 0;
	UE_LOGF(LogGoogleARCore, Log, "ARCore session reset");
}

void FGoogleARCoreDevice::AllocatePassthroughCameraTextures()
{
	FGoogleARCoreCameraConfig CameraConfig;
	GetARCameraConfig(CameraConfig);
	
	TArray<UARCoreCameraTexture*> Textures;
	for (auto Index = 0; Index < NUM_CAMERA_TEXTURES; ++Index)
	{
		if (auto Texture = UARTexture::CreateARTexture<UARCoreCameraTexture>(EARTextureType::CameraImage))
		{
			Texture->Size = { (float)CameraConfig.CameraTextureResolution.X, (float)CameraConfig.CameraTextureResolution.Y };
			Texture->UpdateResource();
			Textures.Add(Texture);
		}
	}
	
	FlushRenderingCommands();
	
	for (auto Texture : Textures)
	{
		auto TextureId = Texture->GetTextureId();
		check(TextureId);
		PassthroughCameraTextures.Add(TextureId, Texture);
		UE_LOGF(LogGoogleARCore, Log, "Created external camera texture of size (%.0f x %.0f) with texture Id (%d) and external texture Guid (%ls)",
			   Texture->Size.X, Texture->Size.Y, TextureId, *Texture->ExternalTextureGuid.ToString());
	}

	// Keep an array of the keys to be able circle through the textures frame by frame when using Vulkan.
	PassthroughCameraTextures.GetKeys(PassthroughCameraTextureKeys);
}

FMatrix FGoogleARCoreDevice::GetPassthroughCameraProjectionMatrix(FIntPoint ViewRectSize) const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return FMatrix::Identity;
	}
	return ARCoreSession->GetLatestFrame()->GetProjectionMatrix();
}

void FGoogleARCoreDevice::GetPassthroughCameraImageUVs(const TArray<float>& InUvs, TArray<float>& OutUVs) const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return;
	}
	ARCoreSession->GetLatestFrame()->TransformDisplayUvCoords(InUvs, OutUVs);
}

int64 FGoogleARCoreDevice::GetPassthroughCameraTimestamp() const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return 0;
	}
	return ARCoreSession->GetLatestFrame()->GetCameraTimestamp();
}

EGoogleARCoreTrackingState FGoogleARCoreDevice::GetTrackingState() const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return EGoogleARCoreTrackingState::StoppedTracking;
	}
	else if (!bIsARCoreSessionRunning)
	{
		return EGoogleARCoreTrackingState::NotTracking;
	}

	return ARCoreSession->GetLatestFrame()->GetCameraTrackingState();
}

EGoogleARCoreTrackingFailureReason FGoogleARCoreDevice::GetTrackingFailureReason() const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return EGoogleARCoreTrackingFailureReason::None;
	}

	return ARCoreSession->GetLatestFrame()->GetCameraTrackingFailureReason();
}

FTransform FGoogleARCoreDevice::GetLatestPose() const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return FTransform::Identity;
	}
	return ARCoreSession->GetLatestFrame()->GetCameraPose();
}

EGoogleARCoreFunctionStatus FGoogleARCoreDevice::GetLatestPointCloud(UGoogleARCorePointCloud*& OutLatestPointCloud) const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return EGoogleARCoreFunctionStatus::NotAvailable;
	}

	return ToARCoreFunctionStatus(ARCoreSession->GetLatestFrame()->GetPointCloud(OutLatestPointCloud));
}

EGoogleARCoreFunctionStatus FGoogleARCoreDevice::AcquireLatestPointCloud(UGoogleARCorePointCloud*& OutLatestPointCloud) const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return EGoogleARCoreFunctionStatus::NotAvailable;
	}

	return ToARCoreFunctionStatus(ARCoreSession->GetLatestFrame()->AcquirePointCloud(OutLatestPointCloud));
}

EGoogleARCoreFunctionStatus FGoogleARCoreDevice::AcquireCameraImage(UGoogleARCoreCameraImage *&OutLatestCameraImage)
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return EGoogleARCoreFunctionStatus::NotAvailable;
	}

	return ToARCoreFunctionStatus(ARCoreSession->GetLatestFrame()->AcquireCameraImage(OutLatestCameraImage));
}

void FGoogleARCoreDevice::TransformARCoordinates2D(EGoogleARCoreCoordinates2DType InputCoordinatesType, const TArray<FVector2D>& InputCoordinates, EGoogleARCoreCoordinates2DType OutputCoordinatesType, TArray<FVector2D>& OutputCoordinates) const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		UE_LOGF(LogGoogleARCore, Log, "Failed to transform ARCoordinate2D due to thers is no valid ARCore session.");
		return;
	}

	return ARCoreSession->GetLatestFrame()->TransformARCoordinates2D(InputCoordinatesType, InputCoordinates, OutputCoordinatesType, OutputCoordinates);
}

FGoogleARCoreLightEstimate FGoogleARCoreDevice::GetLatestLightEstimate() const
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return FGoogleARCoreLightEstimate();
	}

	return ARCoreSession->GetLatestFrame()->GetLightEstimate();
}

void FGoogleARCoreDevice::ARLineTrace(const FVector2D& ScreenPosition, EGoogleARCoreLineTraceChannel TraceChannels, TArray<FARTraceResult>& OutHitResults)
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return;
	}
	OutHitResults.Empty();
	ARCoreSession->GetLatestFrame()->ARLineTrace(ScreenPosition, TraceChannels, OutHitResults);
}

void FGoogleARCoreDevice::ARLineTrace(const FVector& Start, const FVector& End, EGoogleARCoreLineTraceChannel TraceChannels, TArray<FARTraceResult>& OutHitResults)
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return;
	}
	OutHitResults.Empty();
	ARCoreSession->GetLatestFrame()->ARLineTrace(Start, End, TraceChannels, OutHitResults);
}

EGoogleARCoreFunctionStatus FGoogleARCoreDevice::CreateARPin(const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry, USceneComponent* ComponentToPin, const FName DebugName, UARPin*& OutARAnchorObject)
{
	if (!bIsARCoreSessionRunning)
	{
		return EGoogleARCoreFunctionStatus::SessionPaused;
	}

	const FTransform& TrackingToAlignedTracking = ARSystem->GetAlignmentTransform();
	const FTransform PinToTrackingTransform = PinToWorldTransform.GetRelativeTransform(ARSystem->GetXRTrackingSystem()->GetTrackingToWorldTransform()).GetRelativeTransform(TrackingToAlignedTracking);

	EGoogleARCoreFunctionStatus Status = ToARCoreFunctionStatus(ARCoreSession->CreateARAnchor(PinToTrackingTransform, TrackedGeometry, ComponentToPin, DebugName, OutARAnchorObject));

	return Status;
}

bool FGoogleARCoreDevice::TryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InPinName, UARPin*& OutPin)
{
	OutPin = nullptr;
	if (!bIsARCoreSessionRunning)
	{
		return false;
	}

	return ARCoreSession->TryGetOrCreatePinForNativeResource(InNativeResource, InPinName, OutPin);
}

void FGoogleARCoreDevice::RemoveARPin(UARPin* ARAnchorObject)
{
	if (!ARCoreSession.IsValid())
	{
		return;
	}

	ARCoreSession->DetachAnchor(ARAnchorObject);
}

void FGoogleARCoreDevice::GetAllARPins(TArray<UARPin*>& ARCoreAnchorList)
{
	if (!ARCoreSession.IsValid())
	{
		return;
	}
	ARCoreSession->GetAllAnchors(ARCoreAnchorList);
}

void FGoogleARCoreDevice::GetUpdatedARPins(TArray<UARPin*>& ARCoreAnchorList)
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return;
	}
	ARCoreSession->GetLatestFrame()->GetUpdatedAnchors(ARCoreAnchorList);
}

// Functions that are called on Android lifecycle events.
void FGoogleARCoreDevice::OnApplicationCreated()
{
}

void FGoogleARCoreDevice::OnApplicationDestroyed()
{
}

void FGoogleARCoreDevice::OnApplicationPause()
{
	UE_LOGF(LogGoogleARCore, Log, "OnPause Called: %d", bIsARCoreSessionRunning);
	bShouldSessionRestart = bIsARCoreSessionRunning;
	if (bIsARCoreSessionRunning)
	{
		PauseARCoreSession();
	}
}

void FGoogleARCoreDevice::OnApplicationResume()
{
	UE_LOGF(LogGoogleARCore, Log, "OnResume Called: %d", bShouldSessionRestart);
	// Try to ask for permission if it is denied by user.
	if (bShouldSessionRestart)
	{
		bShouldSessionRestart = false;
		StartSession();
	}
}

void FGoogleARCoreDevice::OnApplicationStop()
{
}

void FGoogleARCoreDevice::OnApplicationStart()
{
}

// TODO: we probably don't need this.
void FGoogleARCoreDevice::OnDisplayOrientationChanged()
{
	FGoogleARCoreAndroidHelper::UpdateDisplayRotation();
	bDisplayOrientationChanged = true;
}

UARSessionConfig* FGoogleARCoreDevice::AccessSessionConfig() const
{
	return (ARSystem.IsValid())
		? &ARSystem->AccessSessionConfig()
		: nullptr;
}


EGoogleARCoreFunctionStatus FGoogleARCoreDevice::GetCameraImageIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics)
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return EGoogleARCoreFunctionStatus::NotAvailable;
	}

	return ToARCoreFunctionStatus(ARCoreSession->GetLatestFrame()->GetCameraImageIntrinsics(OutCameraIntrinsics));
}

EGoogleARCoreFunctionStatus FGoogleARCoreDevice::GetCameraTextureIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics)
{
	if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
	{
		return EGoogleARCoreFunctionStatus::NotAvailable;
	}

	return ToARCoreFunctionStatus(ARCoreSession->GetLatestFrame()->GetCameraTextureIntrinsics(OutCameraIntrinsics));
}

UARTexture* FGoogleARCoreDevice::GetLastCameraTexture() const
{
	if (auto Record = PassthroughCameraTextures.Find(LastCameraTextureId))
	{
		return *Record;
	}
	
	return nullptr;
}

UARTexture* FGoogleARCoreDevice::GetDepthTexture() const
{
	return DepthTexture;
}

IYCbCrConversionQuery* FGoogleARCoreDevice::GetLastCameraYCbCrConversionQuery() const
{
	if (auto Record = PassthroughCameraTextures.Find(LastCameraTextureId))
	{
		return *Record;
	}

	return nullptr;
}

void FGoogleARCoreDevice::FInternalGCObject::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ARCoreDevice->DepthTexture);
	Collector.AddReferencedObject(ARCoreDevice->PermissionHandler);
	Collector.AddReferencedObjects(ARCoreDevice->PassthroughCameraTextures);
}
