// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreDelegates.h"
#include "Engine/EngineBaseTypes.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Queue.h"
#include "Containers/Map.h"
#include "UObject/GCObject.h"

#include "GoogleARCoreTypes.h"
#include "GoogleARCoreAPI.h"
#include "GoogleARCorePassthroughCameraRenderer.h"
#include "GoogleARCoreAugmentedImageDatabase.h"


class UARTexture;
class UARCoreCameraTexture;
class UARCoreDepthTexture;

enum class EARCorePermissionStatus : uint8
{
	Unknown,
	Requested,
	Granted,
	Denied,
};

class FGoogleARCoreDevice
{
public:
	static FGoogleARCoreDevice* GetInstance();
	
	FGoogleARCoreDevice();

	EGoogleARCoreAvailability CheckARCoreAPKAvailability();

	EGoogleARCoreAPIStatus RequestInstall(bool bUserRequestedInstall, EGoogleARCoreInstallStatus& OutInstallStatus);

	bool GetIsTrackingTypeSupported(EARSessionType SessionType);

	bool GetIsARCoreSessionRunning();

	FARSessionStatus GetSessionStatus();

	// Get Unreal Units per meter, based off of the current map's VR World to Meters setting.
	float GetWorldToMetersScale();

	// Start ARSession with custom session config.
	void StartARCoreSessionRequest(UARSessionConfig* SessionConfig);

	bool SetARCameraConfig(FGoogleARCoreCameraConfig CameraConfig);

	bool GetARCameraConfig(FGoogleARCoreCameraConfig& OutCurrentCameraConfig);

	bool GetIsFrontCameraSession();

	bool GetShouldInvertCulling();

	// Add image to TargetImageDatabase and return the image index.
	// Return -1 if the image cannot be processed.
	int AddRuntimeAugmentedImage(UGoogleARCoreAugmentedImageDatabase* TargetImageDatabase, const TArray<uint8>& ImageGrayscalePixels,
		int ImageWidth, int ImageHeight, FString ImageName, float ImageWidthInMeter);

	bool AddRuntimeCandidateImage(UARSessionConfig* SessionConfig, const TArray<uint8>& ImageGrayscalePixels, int ImageWidth, int ImageHeight,
		FString FriendlyName, float PhysicalWidth);

	bool GetStartSessionRequestFinished();

	void PauseARCoreSession();

	void ResetARCoreSession();

	void AllocatePassthroughCameraTextures();

	// Passthrough Camera
	FMatrix GetPassthroughCameraProjectionMatrix(FIntPoint ViewRectSize) const;
	void GetPassthroughCameraImageUVs(const TArray<float>& InUvs, TArray<float>& OutUVs) const;
	int64 GetPassthroughCameraTimestamp() const;

	// Frame
	void UpdateGameFrame(UWorld* World);
	EGoogleARCoreTrackingState GetTrackingState() const;
	EGoogleARCoreTrackingFailureReason GetTrackingFailureReason() const;
	FTransform GetLatestPose() const;
	FGoogleARCoreLightEstimate GetLatestLightEstimate() const;
	EGoogleARCoreFunctionStatus GetLatestPointCloud(UGoogleARCorePointCloud*& OutLatestPointCloud) const;
	EGoogleARCoreFunctionStatus AcquireLatestPointCloud(UGoogleARCorePointCloud*& OutLatestPointCloud) const;
	EGoogleARCoreFunctionStatus AcquireCameraImage(UGoogleARCoreCameraImage *&OutLatestCameraImage);

	void TransformARCoordinates2D(EGoogleARCoreCoordinates2DType InputCoordinatesType, const TArray<FVector2D>& InputCoordinates,
		EGoogleARCoreCoordinates2DType OutputCoordinatesType, TArray<FVector2D>& OutputCoordinates) const;

	// Hit test
	void ARLineTrace(const FVector2D& ScreenPosition, EGoogleARCoreLineTraceChannel TraceChannels, TArray<FARTraceResult>& OutHitResults);
	void ARLineTrace(const FVector& Start, const FVector& End, EGoogleARCoreLineTraceChannel TraceChannels, TArray<FARTraceResult>& OutHitResults);

	// Anchor, Planes
	EGoogleARCoreFunctionStatus CreateARPin(const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry, USceneComponent* ComponentToPin, const FName DebugName, UARPin*& OutARAnchorObject);
	bool TryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InPinName, UARPin*& OutPin);
	void RemoveARPin(UARPin* ARAnchorObject);

	void GetAllARPins(TArray<UARPin*>& ARCoreAnchorList);
	void GetUpdatedARPins(TArray<UARPin*>& ARCoreAnchorList);

	template< class T >
	void GetUpdatedTrackables(TArray<T*>& OutARCoreTrackableList)
	{
		if (!ARCoreSession.IsValid() || ARCoreSession->GetLatestFrame() == nullptr)
		{
			return;
		}
		ARCoreSession->GetLatestFrame()->GetUpdatedTrackables<T>(OutARCoreTrackableList);
	}

	template< class T >
	void GetAllTrackables(TArray<T*>& OutARCoreTrackableList)
	{
		if (!ARCoreSession.IsValid())
		{
			return;
		}
		ARCoreSession->GetAllTrackables<T>(OutARCoreTrackableList);
	}

	// Camera Intrinsics
	EGoogleARCoreFunctionStatus GetCameraImageIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics);
	EGoogleARCoreFunctionStatus GetCameraTextureIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics);

	void RunOnGameThread(TFunction<void()> Func)
	{
		RunOnGameThreadQueue.Enqueue(Func);
	}

	void GetRequiredRuntimePermissionsForConfiguration(const UARSessionConfig& Config, TArray<FString>& RuntimePermissions)
	{
		RuntimePermissions.Reset();
		// TODO: check for depth camera when it is supported here.
		RuntimePermissions.Add("android.permission.CAMERA");
	}
	void HandleRuntimePermissionsGranted(const TArray<FString>& Permissions, const TArray<bool>& Granted);

	// Function that is used to call from the Android UI thread:
	void StartSessionWithRequestedConfig();

	TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> GetARSystem();
	void SetARSystem(TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> InARSystem);

	void* GetARSessionRawPointer();
	void* GetGameThreadARFrameRawPointer();
	
	UARTexture* GetLastCameraTexture() const;
	UARTexture* GetDepthTexture() const;

	IYCbCrConversionQuery* GetLastCameraYCbCrConversionQuery() const;
	
	EARCorePermissionStatus CheckAndRequrestPermission(const UARSessionConfig& ConfigurationData);

private:
	// Android lifecycle events.
	void OnApplicationCreated();
	void OnApplicationDestroyed();
	void OnApplicationPause();
	void OnApplicationResume();
	void OnApplicationStart();
	void OnApplicationStop();
	void OnDisplayOrientationChanged();

	// Unreal plugin events.
	void OnModuleLoaded();
	void OnModuleUnloaded();

	TSharedPtr<FGoogleARCoreSession> CreateSession();
	void StartSession();

	friend class FGoogleARCoreAndroidHelper;
	friend class FGoogleARCoreBaseModule;

	UARSessionConfig* AccessSessionConfig() const;

private:
	TSharedPtr<FGoogleARCoreSession> ARCoreSession;
	TSharedPtr<FGoogleARCoreSession> FrontCameraARCoreSession;
	TSharedPtr<FGoogleARCoreSession> BackCameraARCoreSession;
	
	TMap<uint32, TObjectPtr<UARCoreCameraTexture>> PassthroughCameraTextures;
	TArray<uint32> PassthroughCameraTextureKeys;
	
	TObjectPtr<UARCoreDepthTexture> DepthTexture;
	
	FTextureRHIRef LastCameraTexture;
	
	FCriticalSection LastCameraTextureLock;
	
	uint32 LastCameraTextureId = 0;
	
	bool bIsARCoreSessionRunning;
	
	EARCorePermissionStatus PermissionStatus = EARCorePermissionStatus::Unknown;
	
	bool bStartSessionRequested; // User called StartSession
	bool bShouldSessionRestart; // Start tracking on activity start
	bool bARCoreInstallRequested;
	bool bARCoreInstalled;
	float WorldToMeterScale;
	TObjectPtr<class UARCoreAndroidPermissionHandler> PermissionHandler;
	FThreadSafeBool bDisplayOrientationChanged;

	FARSessionStatus CurrentSessionStatus;

	FGoogleARCoreCameraConfig SessionCameraConfig;
	
	TQueue<TFunction<void()>> RunOnGameThreadQueue;

	TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSystem;
	
	// FGoogleARCoreDevice itself cannot be a FGCObject since it's initialize too early.
	// So we use an internal class instead, which can be created only when it's needed.
	class FInternalGCObject : FGCObject
	{
	public:
		FInternalGCObject(FGoogleARCoreDevice* InARCoreDevice) : ARCoreDevice(InARCoreDevice) {}
		void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("FGoogleARCoreDevice::FInternalGCObject");
		}
	private:
		FGoogleARCoreDevice* ARCoreDevice = nullptr;
	};
	friend class FInternalGCObject;
	
	TSharedPtr<FInternalGCObject, ESPMode::ThreadSafe> GCObject;
};
