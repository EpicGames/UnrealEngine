// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreFunctionLibrary.h"
#include "UnrealEngine.h"
#include "Engine/Engine.h"
#include "Misc/AssertionMacros.h"
#include "ARBlueprintLibrary.h"

#include "GoogleARCoreAndroidHelper.h"
#include "GoogleARCoreBaseLogCategory.h"
#include "GoogleARCoreDevice.h"
#include "GoogleARCoreXRTrackingSystem.h"

UARCandidateImage* UGoogleARCoreSessionFunctionLibrary::AddRuntimeCandidateImageFromRawbytes(UARSessionConfig* SessionConfig, const TArray<uint8>& ImageGrayscalePixels,
	int ImageWidth, int ImageHeight, FString FriendlyName, float PhysicalWidth, UTexture2D* CandidateTexture /*= nullptr*/)
{
	if (auto TrackingSystem = FGoogleARCoreXRTrackingSystem::GetInstance())
	{
		if (TrackingSystem->AddRuntimeGrayscaleImage(SessionConfig, ImageGrayscalePixels, ImageWidth, ImageHeight, FriendlyName, PhysicalWidth))
		{
			float PhysicalHeight = PhysicalWidth / ImageWidth * ImageHeight;
			UARCandidateImage* NewCandidateImage = UARCandidateImage::CreateNewARCandidateImage(CandidateTexture, FriendlyName, PhysicalWidth, PhysicalHeight, EARCandidateImageOrientation::Landscape);
			SessionConfig->AddCandidateImage(NewCandidateImage);
			return NewCandidateImage;
		}
	}

	return nullptr;
}
