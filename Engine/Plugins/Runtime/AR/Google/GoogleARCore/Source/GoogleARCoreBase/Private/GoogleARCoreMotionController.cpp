// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreMotionController.h"
#include "GoogleARCoreBaseLogCategory.h"
#include "GoogleARCoreDevice.h"

FName FGoogleARCoreMotionController::ARCoreMotionSourceId(TEXT("ARCoreDevice"));

FGoogleARCoreMotionController::FGoogleARCoreMotionController()
	: ARCoreDeviceInstance(nullptr)
{
	ARCoreDeviceInstance = FGoogleARCoreDevice::GetInstance();
}

void FGoogleARCoreMotionController::RegisterController()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
}

void FGoogleARCoreMotionController::UnregisterController()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
}

FName FGoogleARCoreMotionController::GetMotionControllerDeviceTypeName() const
{
	static const FName DeviceName(TEXT("ARcoreCameraMotionController"));
	return DeviceName;
}

bool FGoogleARCoreMotionController::GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const
{
	if (MotionSource != ARCoreMotionSourceId)
	{
		return false;
	}

	// Only update in game thread and opt out updating on render thread since we only update the ARCore pose on game thread.
	if (IsInGameThread() && ARCoreDeviceInstance->GetTrackingState() == EGoogleARCoreTrackingState::Tracking)
	{
		FTransform LatestPose = ARCoreDeviceInstance->GetLatestPose();
		OutOrientation = FRotator(LatestPose.GetRotation());
		OutPosition = LatestPose.GetTranslation();

		return true;
	}

	return false;
}

ETrackingStatus FGoogleARCoreMotionController::GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const
{
	if (MotionSource == ARCoreMotionSourceId && ARCoreDeviceInstance->GetTrackingState() == EGoogleARCoreTrackingState::Tracking)
	{
		return ETrackingStatus::Tracked;
	}
	else
	{
		return ETrackingStatus::NotTracked;
	}
}

void FGoogleARCoreMotionController::EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const
{
	SourcesOut.Add(ARCoreMotionSourceId);
}