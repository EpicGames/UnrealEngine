// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMotionController.h"
#include "Features/IModularFeatures.h"


class FGoogleARCoreDevice;

class FGoogleARCoreMotionController : public IMotionController
{
public:
	FGoogleARCoreMotionController();

	void RegisterController();

	void UnregisterController();

	/**
	* Returns the device type of the controller.
	*
	* @return	Device type of the controller.
	*/
	virtual FName GetMotionControllerDeviceTypeName() const override;

	/**
	* Returns the calibration-space orientation of the requested controller's hand.
	*
	* @param ControllerIndex	The Unreal controller (player) index of the controller set
	* @param MotionSource		Which source, within the motion controller to get the orientation and position for
	* @param OutOrientation	(out) If tracked, the orientation (in calibrated-space) of the controller in the specified hand
	* @param OutPosition		(out) If tracked, the position (in calibrated-space) of the controller in the specified hand
	* @param WorldToMetersScale The world scaling factor.
	*
	* @return					True if the device requested is valid and tracked, false otherwise
	*/
	virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const override;

	virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const override
	{
		OutbProvidedLinearVelocity = false;
		OutbProvidedAngularVelocity = false;
		OutbProvidedLinearAcceleration = false;
		return GetControllerOrientationAndPosition(ControllerIndex, MotionSource, OutOrientation, OutPosition, WorldToMetersScale);
	}

	virtual bool GetControllerOrientationAndPositionForTime(const int32 ControllerIndex, const FName MotionSource, FTimespan Time, bool& OutTimeWasUsed, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const override
	{
		OutTimeWasUsed = false;
		OutbProvidedLinearVelocity = false;
		OutbProvidedAngularVelocity = false;
		OutbProvidedLinearAcceleration = false;
		return GetControllerOrientationAndPosition(ControllerIndex, MotionSource, OutOrientation, OutPosition, WorldToMetersScale);
	}

	/**
	* Returns the tracking status (e.g. not tracked, intertial-only, fully tracked) of the specified controller
	*
	* @return	Tracking status of the specified controller, or ETrackingStatus::NotTracked if the device is not found
	*/
	virtual ETrackingStatus GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const override;

	/**
	* Called to request the motion sources that this IMotionController provides
	*
	* @param Sources	A motion source enumerator object that IMotionControllers can add source names to
	*/
	virtual void EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const override;

	/**
	* Returns a custom names parameter value
	*
	* @param MotionSource		The name of the motion source we want parameters for
	* @param ParameterName		The specific value we are looking for
	* @param bOutValueFound	(out) Whether the parameter could be found
	*
	* @return			The value of the parameter
	*/
	virtual float GetCustomParameterValue(const FName MotionSource, FName ParameterName, bool& bOutValueFound) const override { bOutValueFound = false;  return 0.f; }

	virtual bool GetHandJointPosition(const FName MotionSource, int jointIndex, FVector& OutPosition) const { return false; }

private:
	static FName ARCoreMotionSourceId;

	FGoogleARCoreDevice* ARCoreDeviceInstance;
};