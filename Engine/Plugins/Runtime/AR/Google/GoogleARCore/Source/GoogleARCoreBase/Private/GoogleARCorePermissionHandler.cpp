// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCorePermissionHandler.h"
#include "AndroidPermissionFunctionLibrary.h"
#include "AndroidPermissionCallbackProxy.h"
#include "GoogleARCoreDevice.h"

UARCoreAndroidPermissionHandler::UARCoreAndroidPermissionHandler(const FObjectInitializer& Init) : Super(Init)
{
}

bool UARCoreAndroidPermissionHandler::CheckRuntimePermission(const FString& RuntimePermission)
{
	return UAndroidPermissionFunctionLibrary::CheckPermission(RuntimePermission);
}

void UARCoreAndroidPermissionHandler::RequestRuntimePermissions(const TArray<FString>& RuntimePermissions)
{
	UAndroidPermissionCallbackProxy::GetInstance()->OnPermissionsGrantedDynamicDelegate.AddDynamic(this, &UARCoreAndroidPermissionHandler::OnPermissionsGranted);
	UAndroidPermissionFunctionLibrary::AcquirePermissions(RuntimePermissions);
}

void UARCoreAndroidPermissionHandler::OnPermissionsGranted(const TArray<FString> &Permissions, const TArray<bool>& Granted)
{
	UAndroidPermissionCallbackProxy::GetInstance()->OnPermissionsGrantedDynamicDelegate.RemoveDynamic(this, &UARCoreAndroidPermissionHandler::OnPermissionsGranted);
	FGoogleARCoreDevice::GetInstance()->HandleRuntimePermissionsGranted(Permissions, Granted);
}

