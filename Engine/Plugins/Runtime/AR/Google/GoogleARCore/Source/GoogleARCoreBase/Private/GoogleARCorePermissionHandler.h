// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "GoogleARCorePermissionHandler.generated.h"

UCLASS()
class UARCoreAndroidPermissionHandler: public UObject
{
	GENERATED_UCLASS_BODY()
public:

	static bool CheckRuntimePermission(const FString& RuntimePermission);

	void RequestRuntimePermissions(const TArray<FString>& RuntimePermissions);

	UFUNCTION()
	void OnPermissionsGranted(const TArray<FString>& Permissions, const TArray<bool>& Granted);
};
