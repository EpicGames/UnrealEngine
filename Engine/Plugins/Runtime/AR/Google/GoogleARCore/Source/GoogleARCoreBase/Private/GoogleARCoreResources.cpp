// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreResources.h"
#include "ARTrackable.h"
#include "ARSessionConfig.h"
#include "GoogleARCoreBaseLogCategory.h"
#include "GoogleARCoreXRTrackingSystem.h"
#include "ARBlueprintLibrary.h"

#if PLATFORM_ANDROID
// Defined in GoogleARCoreAPI.cpp
extern EARTrackingState ToARTrackingState(ArTrackingState State);
extern FTransform ARCorePoseToUnrealTransform(ArPose* ArPoseHandle, const ArSession* SessionHandle, float WorldToMeterScale);
extern bool CheckIsSessionValid(FString TypeName, const TWeakPtr<FGoogleARCoreSession>& SessionPtr);

EARTrackingState FGoogleARCoreTrackableResource::GetTrackingState()
{
	EARTrackingState TrackingState = EARTrackingState::StoppedTracking;
	if (CheckIsSessionValid("ARCoreTrackable", Session))
	{
		ArTrackingState ARTrackingState = ArTrackingState::AR_TRACKING_STATE_STOPPED;
		ArTrackable_getTrackingState(Session.Pin()->GetHandle(), TrackableHandle, &ARTrackingState);
		TrackingState = ToARTrackingState(ARTrackingState);
	}
	return TrackingState;
}

void FGoogleARCoreTrackableResource::UpdateGeometryData()
{
	TrackedGeometry->UpdateTrackingState(GetTrackingState());
}

void FGoogleARCoreTrackableResource::ResetNativeHandle(ArTrackable* InTrackableHandle)
{
	if (TrackableHandle != nullptr)
	{
		ArTrackable_release(TrackableHandle);
	}
	TrackableHandle = InTrackableHandle;

	UpdateGeometryData();
}

void FGoogleARCoreTrackedPlaneResource::UpdateGeometryData()
{
	FGoogleARCoreTrackableResource::UpdateGeometryData();

	UARPlaneGeometry* PlaneGeometry = CastChecked<UARPlaneGeometry>(TrackedGeometry);

	if (!CheckIsSessionValid("ARCorePlane", Session) || TrackedGeometry->GetTrackingState() == EARTrackingState::StoppedTracking)
	{
		return;
	}

	TSharedPtr<FGoogleARCoreSession> SessionPtr = Session.Pin();

	FTransform LocalToTrackingTransform;
	FVector Extent = FVector::ZeroVector;

	// Get Center Transform
	ArPose* ARPoseHandle = nullptr;
	ArPose_create(SessionPtr->GetHandle(), nullptr, &ARPoseHandle);
	ArPlane_getCenterPose(SessionPtr->GetHandle(), GetPlaneHandle(), ARPoseHandle);
	LocalToTrackingTransform = ARCorePoseToUnrealTransform(ARPoseHandle, SessionPtr->GetHandle(), SessionPtr->GetWorldToMeterScale());
	ArPose_destroy(ARPoseHandle);

	// Get Plane Extents
	float ARCorePlaneExtentX = 0; // X is right vector
	float ARCorePlaneExtentZ = 0; // Z is backward vector
	ArPlane_getExtentX(SessionPtr->GetHandle(), GetPlaneHandle(), &ARCorePlaneExtentX);
	ArPlane_getExtentZ(SessionPtr->GetHandle(), GetPlaneHandle(), &ARCorePlaneExtentZ);

	// Convert OpenGL axis to Unreal axis.
	// Unreal TrackedPlaneGeometry extent is the length from the plane center to edge.
	Extent = FVector(FMath::Abs(ARCorePlaneExtentZ / 2.0f), FMath::Abs(ARCorePlaneExtentX / 2.0f), 0) * SessionPtr->GetWorldToMeterScale();

	// Update Boundary Polygon
	int PolygonSize = 0;
	ArPlane_getPolygonSize(SessionPtr->GetHandle(), GetPlaneHandle(), &PolygonSize);

	TArray<FVector> BoundaryPolygon;
	BoundaryPolygon.Empty(PolygonSize / 2);

	if (PolygonSize != 0)
	{
		TArray<float> PolygonPointsXZ;
		PolygonPointsXZ.SetNumUninitialized(PolygonSize);
		ArPlane_getPolygon(SessionPtr->GetHandle(), GetPlaneHandle(), PolygonPointsXZ.GetData());

		for (int i = 0; i < PolygonSize / 2; i++)
		{
			const FVector PointInLocalSpace(-PolygonPointsXZ[2 * i + 1] * SessionPtr->GetWorldToMeterScale(), PolygonPointsXZ[2 * i] * SessionPtr->GetWorldToMeterScale(), 0.0f);
			BoundaryPolygon.Add(PointInLocalSpace);
		}
	}

	ArPlaneType PlaneType = ArPlaneType::AR_PLANE_HORIZONTAL_UPWARD_FACING;
	ArPlane_getType(SessionPtr->GetHandle(), GetPlaneHandle(), &PlaneType);

	switch (PlaneType)
	{
		// TODO: Do we actually want to set the object classification? ARCore docs
		// says this is ok, but the other AR APIs are explicit about the plane classification
		case ArPlaneType::AR_PLANE_HORIZONTAL_UPWARD_FACING:
		{
			PlaneGeometry->SetOrientation(EARPlaneOrientation::Horizontal);
			//PlaneGeometry->SetObjectClassification(EARObjectClassification::Floor);
			break;
		}
		case ArPlaneType::AR_PLANE_HORIZONTAL_DOWNWARD_FACING:
		{
			PlaneGeometry->SetOrientation(EARPlaneOrientation::Horizontal);
			//PlaneGeometry->SetObjectClassification(EARObjectClassification::Ceiling);
			break;
		}
		case ArPlaneType::AR_PLANE_VERTICAL:
		{
			PlaneGeometry->SetOrientation(EARPlaneOrientation::Vertical);
			//PlaneGeometry->SetObjectClassification(EARObjectClassification::Wall);
			break;
		}
		default:
		{
			UE_LOGF(LogGoogleARCore, Warning, "Unsupported ArPlaneType: %d.", PlaneType);
			PlaneGeometry->SetOrientation(EARPlaneOrientation::Horizontal);
			//PlaneGeometry->SetObjectClassification(EARObjectClassification::Floor);
			break;
		}
	}

	ArPlane* SubsumedByPlaneHandle = nullptr;
	ArPlane_acquireSubsumedBy(SessionPtr->GetHandle(), GetPlaneHandle(), &SubsumedByPlaneHandle);
	ArTrackable* TrackableHandle = reinterpret_cast<ArTrackable*>(SubsumedByPlaneHandle);

	UARPlaneGeometry* SubsumedByPlane = SubsumedByPlaneHandle  == nullptr? nullptr : SessionPtr->GetUObjectManager()->GetTrackableFromHandle<UARPlaneGeometry>(TrackableHandle, SessionPtr.Get());

	const uint32 FrameNum = SessionPtr->GetFrameNum();
	const int64 TimeStamp = SessionPtr->GetLatestFrame()->GetCameraTimestamp();

	PlaneGeometry->UpdateTrackedGeometry(SessionPtr->GetARSystem(), FrameNum, static_cast<double>(TimeStamp), LocalToTrackingTransform, SessionPtr->GetARSystem()->GetAlignmentTransform(), FVector::ZeroVector, Extent, BoundaryPolygon, SubsumedByPlane);
	PlaneGeometry->SetDebugName(FName(TEXT("ARCorePlane")));
}

void FGoogleARCoreTrackedPointResource::UpdateGeometryData()
{
	FGoogleARCoreTrackableResource::UpdateGeometryData();

	UARTrackedPoint* TrackedPoint = CastChecked<UARTrackedPoint>(TrackedGeometry);

	if (!CheckIsSessionValid("ARCoreTrackablePoint", Session) || TrackedGeometry->GetTrackingState() == EARTrackingState::StoppedTracking)
	{
		return;
	}

	TSharedPtr<FGoogleARCoreSession> SessionPtr = Session.Pin();

	ArPose* ARPoseHandle = nullptr;
	ArPose_create(SessionPtr->GetHandle(), nullptr, &ARPoseHandle);
	ArPoint_getPose(SessionPtr->GetHandle(), GetPointHandle(), ARPoseHandle);
	FTransform PointPose = ARCorePoseToUnrealTransform(ARPoseHandle, SessionPtr->GetHandle(), SessionPtr->GetWorldToMeterScale());
	// TODO: hook up Orientation valid.
	bool bIsPoseOrientationValid = false;
	ArPose_destroy(ARPoseHandle);

	uint32 FrameNum = SessionPtr->GetFrameNum();
	int64 TimeStamp = SessionPtr->GetLatestFrame()->GetCameraTimestamp();
	TrackedPoint->UpdateTrackedGeometry(SessionPtr->GetARSystem(), FrameNum, static_cast<double>(TimeStamp), PointPose, SessionPtr->GetARSystem()->GetAlignmentTransform());
	TrackedPoint->SetDebugName(FName(TEXT("ARCoreTrackedPoint")));
}

void FGoogleARCoreAugmentedImageResource::UpdateGeometryData()
{
	FGoogleARCoreTrackableResource::UpdateGeometryData();
	
	TSharedPtr<FGoogleARCoreSession> SessionPtr = Session.Pin();
	const auto ImageHandle = GetImageHandle();
	const auto SessionHandle = SessionPtr->GetHandle();
	
	if (TrackedGeometry->GetTrackingState() == EARTrackingState::Tracking)
	{
		ArAugmentedImageTrackingMethod TrackingMethod = AR_AUGMENTED_IMAGE_TRACKING_METHOD_NOT_TRACKING;
		ArAugmentedImage_getTrackingMethod(SessionHandle, ImageHandle, &TrackingMethod);
		
		switch (TrackingMethod)
		{
			case AR_AUGMENTED_IMAGE_TRACKING_METHOD_FULL_TRACKING:
				// The Augmented Image is currently being tracked using the camera image.
				TrackedGeometry->UpdateTrackingState(EARTrackingState::Tracking);
				break;
				
			case AR_AUGMENTED_IMAGE_TRACKING_METHOD_LAST_KNOWN_POSE:
				// The Augmented Image is currently being tracked based on its last known pose,
				// because it can no longer be tracked using the camera image.
				TrackedGeometry->UpdateTrackingState(EARTrackingState::NotTracking);
				break;
				
			case AR_AUGMENTED_IMAGE_TRACKING_METHOD_NOT_TRACKING:
				// The Augmented Image is not currently being tracked.
				TrackedGeometry->UpdateTrackingState(EARTrackingState::StoppedTracking);
				break;
		}
	}
	
	UGoogleARCoreAugmentedImage* AugmentedImage = CastChecked<UGoogleARCoreAugmentedImage>(TrackedGeometry);

	if (!CheckIsSessionValid("ARCoreTrackableImage", Session) || TrackedGeometry->GetTrackingState() != EARTrackingState::Tracking)
	{
		return;
	}

	FTransform LocalToTrackingTransform;
	FVector2D EstimatedSize = FVector2D::ZeroVector;

	// Get Center Transform
	ArPose* ARPoseHandle = nullptr;
	ArPose_create(SessionHandle, nullptr, &ARPoseHandle);
	ArAugmentedImage_getCenterPose(SessionHandle, ImageHandle, ARPoseHandle);
	LocalToTrackingTransform = ARCorePoseToUnrealTransform(ARPoseHandle, SessionHandle, SessionPtr->GetWorldToMeterScale());
	ArPose_destroy(ARPoseHandle);

	// Get AugmentedImage Extents
	float ARCoreAugmentedImageExtentX = 0; // X is right vector
	float ARCoreAugmentedImageExtentZ = 0; // Z is backward vector
	ArAugmentedImage_getExtentX(SessionHandle, ImageHandle, &ARCoreAugmentedImageExtentX);
	ArAugmentedImage_getExtentZ(SessionHandle, ImageHandle, &ARCoreAugmentedImageExtentZ);

	int32 ImageIndex = 0;
	ArAugmentedImage_getIndex(SessionHandle, ImageHandle, &ImageIndex);

	// Convert extents to estimated size where x is the width and y is the height.
	EstimatedSize = FVector2D(FMath::Abs(ARCoreAugmentedImageExtentX), FMath::Abs(ARCoreAugmentedImageExtentZ)) * SessionPtr->GetWorldToMeterScale();

	uint32 FrameNum = SessionPtr->GetFrameNum();
	int64 TimeStamp = SessionPtr->GetLatestFrame()->GetCameraTimestamp();

	char *ImageName = nullptr;
	ArAugmentedImage_acquireName(SessionHandle, ImageHandle, &ImageName);

	UARCandidateImage* TargetCandidateImage = nullptr;

	if (SessionPtr->GetCurrentSessionConfig()->GetCandidateImageList().Num() > 0)
	{
		TargetCandidateImage = SessionPtr->GetCurrentSessionConfig()->GetCandidateImageList()[ImageIndex];
	}

	AugmentedImage->UpdateTrackedGeometry(
		SessionPtr->GetARSystem(), FrameNum,
		static_cast<double>(TimeStamp), LocalToTrackingTransform,
		SessionPtr->GetARSystem()->GetAlignmentTransform(),
		EstimatedSize, TargetCandidateImage,
		ImageIndex, ImageName);

	ArString_release(ImageName);

	AugmentedImage->SetDebugName(FName(TEXT("ARCoreAugmentedImage")));
}

void FGoogleARCoreAugmentedFaceResource::UpdateGeometryData()
{
	FGoogleARCoreTrackableResource::UpdateGeometryData();

	ArAugmentedFace* FaceHandle = GetFaceHandle();
	UGoogleARCoreAugmentedFace* AugmentedFace = CastChecked<UGoogleARCoreAugmentedFace>(TrackedGeometry);

	if (!CheckIsSessionValid("ARCoreAugmentedFace", Session) || TrackedGeometry->GetTrackingState() == EARTrackingState::StoppedTracking)
	{
		return;
	}

	TSharedPtr<FGoogleARCoreSession> SessionPtr = Session.Pin();
	double TimeStamp = static_cast<double>(SessionPtr->GetLatestFrame()->GetCameraTimestamp());

	ArPose* ARPoseHandle = nullptr;
	ArPose_create(SessionPtr->GetHandle(), nullptr, &ARPoseHandle);
	ArAugmentedFace_getCenterPose(SessionPtr->GetHandle(), FaceHandle, ARPoseHandle);
	FTransform LocalToTrackingTransform = ARCorePoseToUnrealTransform(ARPoseHandle, SessionPtr->GetHandle(), SessionPtr->GetWorldToMeterScale());

	const float* VerticesHandle = nullptr;
	const uint16* IndicesHandle = nullptr;
	const float* UVHandle = nullptr;
	int VerticesNumber = 0;
	int IndicesNumber = 0;
	int UVNumber = 0;

	// Prepare vertex buffer
	ArAugmentedFace_getMeshVertices(SessionPtr->GetHandle(), FaceHandle, &VerticesHandle, &VerticesNumber);
	TArray<FVector> Vertices;
	Vertices.AddUninitialized(VerticesNumber);
	FMemory::Memcpy(reinterpret_cast<float*>(Vertices.GetData()), VerticesHandle, VerticesNumber * 3 * sizeof(float));

	for (int i = 0; i < VerticesNumber; i++)
	{
		FVector Vert = Vertices[i];
		Vertices[i].X = -Vert.Z;
		Vertices[i].Y = Vert.X;
		Vertices[i].Z = Vert.Y;
		Vertices[i] = Vertices[i] * SessionPtr->GetWorldToMeterScale();
	}

	// Prepare index buffer
	TArray<int32> Indices = AugmentedFace->GetIndexBuffer();
	if(Indices.Num() == 0)
	{
		ArAugmentedFace_getMeshTriangleIndices(SessionPtr->GetHandle(), FaceHandle, &IndicesHandle, &IndicesNumber);
		IndicesNumber *= 3;

		// We need to convert index value to int32.
		Indices.AddUninitialized(IndicesNumber);
		for (int i = 0; i < IndicesNumber; i += 3)
		{
			Indices[i] = static_cast<int32>(IndicesHandle[i]);
			Indices[i + 1] = static_cast<int32>(IndicesHandle[i + 1]);
			Indices[i + 2] = static_cast<int32>(IndicesHandle[i + 2]);
		}
	}
	
	// https://developers.google.com/ar/reference/c/group/augmented-face?hl=zh-CN#araugmentedface_getmeshtexturecoordinates
	// Cache the UV data as it never changes
	static TArray<FVector2D> CachedFaceUV;
	if (CachedFaceUV.Num() != Vertices.Num())
	{
		ArAugmentedFace_getMeshTextureCoordinates(SessionPtr->GetHandle(), FaceHandle, &UVHandle, &UVNumber);
		CachedFaceUV.AddUninitialized(UVNumber);
		FMemory::Memcpy(reinterpret_cast<float*>(CachedFaceUV.GetData()), UVHandle, UVNumber * sizeof(float) * 2);
		for (int i = 0; i < CachedFaceUV.Num(); i++)
		{
			// Flip the UV in the vertical direction to make it consistent with ARKit
			CachedFaceUV[i].Y = 1.f - CachedFaceUV[i].Y;
		}
	}
	
	FARBlendShapeMap EmptyBlendShape;
	
	AugmentedFace->UpdateFaceGeometry(SessionPtr->GetARSystem(), SessionPtr->GetFrameNum(), TimeStamp,
		LocalToTrackingTransform, SessionPtr->GetARSystem()->GetAlignmentTransform(),
		EmptyBlendShape, Vertices, Indices, CachedFaceUV,
		FTransform::Identity, FTransform::Identity, FVector::ForwardVector);

	// Update augmented face region
	TMap<EGoogleARCoreAugmentedFaceRegion, FTransform> RegionLocalToTrackingTransform;

	FTransform RegionLocalToTracking;
	// Nose
	ArAugmentedFace_getRegionPose(SessionPtr->GetHandle(), FaceHandle, AR_AUGMENTED_FACE_REGION_NOSE_TIP, ARPoseHandle);
	RegionLocalToTracking = ARCorePoseToUnrealTransform(ARPoseHandle, SessionPtr->GetHandle(), SessionPtr->GetWorldToMeterScale());
	RegionLocalToTrackingTransform.Add(EGoogleARCoreAugmentedFaceRegion::NoseTip, RegionLocalToTracking);

	// Forehead left
	ArAugmentedFace_getRegionPose(SessionPtr->GetHandle(), FaceHandle, AR_AUGMENTED_FACE_REGION_FOREHEAD_LEFT, ARPoseHandle);
	RegionLocalToTracking = ARCorePoseToUnrealTransform(ARPoseHandle, SessionPtr->GetHandle(), SessionPtr->GetWorldToMeterScale());
	RegionLocalToTrackingTransform.Add(EGoogleARCoreAugmentedFaceRegion::ForeheadLeft, RegionLocalToTracking);

	// Forehead right
	ArAugmentedFace_getRegionPose(SessionPtr->GetHandle(), FaceHandle, AR_AUGMENTED_FACE_REGION_FOREHEAD_RIGHT, ARPoseHandle);
	RegionLocalToTracking = ARCorePoseToUnrealTransform(ARPoseHandle, SessionPtr->GetHandle(), SessionPtr->GetWorldToMeterScale());
	RegionLocalToTrackingTransform.Add(EGoogleARCoreAugmentedFaceRegion::ForeheadRight, RegionLocalToTracking);

	AugmentedFace->UpdateRegionTransforms(RegionLocalToTrackingTransform);

	ArPose_destroy(ARPoseHandle);
}
#endif // PLATFORM_ANDROID
