// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ARTypes.h"
#include "GoogleARCoreAPI.h"

#if PLATFORM_ANDROID
#include "arcore_c_api.h"
#endif

class FGoogleARCoreTrackableResource : public IARRef
{
public:
	// IARRef interface
	virtual void AddRef() override { }

	virtual void RemoveRef() override
	{
#if PLATFORM_ANDROID
		ArTrackable_release(TrackableHandle);
		TrackableHandle = nullptr;
#endif
	}

#if PLATFORM_ANDROID
public:
	FGoogleARCoreTrackableResource(TSharedPtr<FGoogleARCoreSession> InSession, ArTrackable* InTrackableHandle, UARTrackedGeometry* InTrackedGeometry)
		: Session(InSession)
		, TrackableHandle(InTrackableHandle)
		, TrackedGeometry(InTrackedGeometry)
	{
		ensure(TrackableHandle != nullptr);
	}

	virtual ~FGoogleARCoreTrackableResource()
	{
		ArTrackable_release(TrackableHandle);
		TrackableHandle = nullptr;
	}

	EARTrackingState GetTrackingState();

	virtual void UpdateGeometryData();

	TWeakPtr<FGoogleARCoreSession> GetSession() { return Session; }
	ArTrackable* GetNativeHandle() { return TrackableHandle; }

	void ResetNativeHandle(ArTrackable* InTrackableHandle);

protected:
	TWeakPtr<FGoogleARCoreSession> Session;
	ArTrackable* TrackableHandle;
	UARTrackedGeometry* TrackedGeometry;
#endif
};

class FGoogleARCoreTrackedPlaneResource : public FGoogleARCoreTrackableResource
{
public:
#if PLATFORM_ANDROID
	FGoogleARCoreTrackedPlaneResource(TSharedPtr<FGoogleARCoreSession> InSession, ArTrackable* InTrackableHandle, UARTrackedGeometry* InTrackedGeometry)
		: FGoogleARCoreTrackableResource(InSession, InTrackableHandle, InTrackedGeometry)
	{
		ensure(TrackableHandle != nullptr);
	}

	void UpdateGeometryData() override;

	ArPlane* GetPlaneHandle() { return reinterpret_cast<ArPlane*>(TrackableHandle); }
#endif
};

class FGoogleARCoreTrackedPointResource : public FGoogleARCoreTrackableResource
{
public:
#if PLATFORM_ANDROID
	FGoogleARCoreTrackedPointResource(TSharedPtr<FGoogleARCoreSession> InSession, ArTrackable* InTrackableHandle, UARTrackedGeometry* InTrackedGeometry)
		: FGoogleARCoreTrackableResource(InSession, InTrackableHandle, InTrackedGeometry)
	{
		ensure(TrackableHandle != nullptr);
	}

	void UpdateGeometryData() override;

	ArPoint* GetPointHandle() { return reinterpret_cast<ArPoint*>(TrackableHandle); }
#endif
};

class FGoogleARCoreAugmentedImageResource : public FGoogleARCoreTrackableResource
{
public:
#if PLATFORM_ANDROID
	FGoogleARCoreAugmentedImageResource(TSharedPtr<FGoogleARCoreSession> InSession, ArTrackable* InTrackableHandle, UARTrackedGeometry* InTrackedGeometry)
		: FGoogleARCoreTrackableResource(InSession, InTrackableHandle, InTrackedGeometry)
	{
		ensure(TrackableHandle != nullptr);
	}

	void UpdateGeometryData() override;

	ArAugmentedImage* GetImageHandle() { return reinterpret_cast<ArAugmentedImage*>(TrackableHandle); }
#endif
};

class FGoogleARCoreAugmentedFaceResource : public FGoogleARCoreTrackableResource
{
public:
#if PLATFORM_ANDROID
	FGoogleARCoreAugmentedFaceResource(TSharedPtr<FGoogleARCoreSession> InSession, ArTrackable* InTrackableHandle, UARTrackedGeometry* InTrackedGeometry)
		: FGoogleARCoreTrackableResource(InSession, InTrackableHandle, InTrackedGeometry)
	{
		ensure(TrackableHandle != nullptr);
	}

	void UpdateGeometryData() override;

	ArAugmentedFace* GetFaceHandle() { return reinterpret_cast<ArAugmentedFace*>(TrackableHandle); }
#endif
};
