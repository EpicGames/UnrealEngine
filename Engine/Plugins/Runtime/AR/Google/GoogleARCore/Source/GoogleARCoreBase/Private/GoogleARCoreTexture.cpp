// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreTexture.h"
#include "GoogleARCoreCameraTextureResourceVulkan.h"
#include "TextureResource.h"
#include "RHICommandList.h"
#include "ExternalTexture.h"

class FARCoreCameraTextureResource : public FTextureResource
{
public:
	FARCoreCameraTextureResource(const FGuid& InExternalTextureGuid, uint32 InSizeX, uint32 InSizeY)
	: ExternalTextureGuid(InExternalTextureGuid)
	, SizeX(InSizeX)
	, SizeY(InSizeY)
	{}
	
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
		SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FARCoreCameraTextureResource"), 1, 1, PF_R8G8B8A8)
			.SetFlags(ETextureCreateFlags::External | ETextureCreateFlags::SRGB);

		TextureRHI = RHICmdList.CreateTexture(Desc);

		RHICmdList.EnqueueLambda([this](FRHICommandListBase&)
		{
			void* NativeResource = TextureRHI->GetNativeResource();
			check(NativeResource);
			TextureId = *reinterpret_cast<uint32*>(NativeResource);
		});
		RHICmdList.GetAsImmediate().ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		
		FExternalTextureRegistry::Get().RegisterExternalTexture(ExternalTextureGuid, TextureRHI, SamplerStateRHI);
	}
	
	virtual void ReleaseRHI() override
	{
		FExternalTextureRegistry::Get().UnregisterExternalTexture(ExternalTextureGuid);
		FTextureResource::ReleaseRHI();
	}
	
	virtual uint32 GetSizeX() const override
	{
		return SizeX;
	}
	
	virtual uint32 GetSizeY() const override
	{
		return SizeY;
	}
	
	uint32 GetTextureId() const { return TextureId; }
	
private:
	uint32 TextureId = 0;
	const FGuid ExternalTextureGuid;
	const uint32 SizeX;
	const uint32 SizeY;
};

UARCoreCameraTexture::UARCoreCameraTexture(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	ExternalTextureGuid = FGuid::NewGuid();
}

FTextureResource* UARCoreCameraTexture::CreateResource()
{
#if PLATFORM_ANDROID
	if (FAndroidMisc::ShouldUseVulkan())
	{
		return new FARCoreCameraTextureResourceVulkan(ExternalTextureGuid, VukanHardwareBuffer, Size.X, Size.Y);
	}
#endif

	return new FARCoreCameraTextureResource(ExternalTextureGuid, Size.X, Size.Y);
}

uint32 UARCoreCameraTexture::GetTextureId() const
{
#if PLATFORM_ANDROID
	if (FAndroidMisc::ShouldUseVulkan())
	{
		if (const auto* MyResource = static_cast<const FARCoreCameraTextureResourceVulkan*>(GetResource()))
		{
			return MyResource->GetTextureId();
		}
		return 0;
	}
#endif

	if (const FARCoreCameraTextureResource* MyResource = static_cast<const FARCoreCameraTextureResource*>(GetResource()))
	{
		return MyResource->GetTextureId();
	}
	
	return 0;
}

void UARCoreCameraTexture::UpdateCameraImage(void* InVukanHardwareBuffer)
{
	if (InVukanHardwareBuffer)
	{
		VukanHardwareBuffer = InVukanHardwareBuffer;
		UpdateResource();
	}
}

FYCbCrConversion UARCoreCameraTexture::GetYCbCrConversion_RenderThread() const
{
#if PLATFORM_ANDROID
	if (FAndroidMisc::ShouldUseVulkan())
	{
		if (const auto* MyResource = static_cast<const FARCoreCameraTextureResourceVulkan*>(GetResource()))
		{
			return MyResource->GetCameraYCbCrConversion();
		}
	}
#endif

	return {};
}

class FARCoreDepthTextureResource : public FTextureResource
{
public:
	FARCoreDepthTextureResource(UARCoreDepthTexture* InOwner, uint32 InSizeX, uint32 InSizeY)
	: Owner(InOwner)
	, SizeX(InSizeX)
	, SizeY(InSizeY)
	{
		bSRGB = InOwner->SRGB;
	}
	
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
		SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
		
		FString Name = Owner->GetName();

		// Our source data is 16bits integer so PF_R32_FLOAT is a bit wasteful
		// But PF_R16_UINT doesn't work in the material and it's not easy to convert the data to PF_R16F on the CPU
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(*Name, SizeX, SizeY, PF_R32_FLOAT)
			.SetFlags(ETextureCreateFlags::ShaderResource);

		TextureRHI = RHICmdList.CreateTexture(Desc);

		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);
	}
	
	virtual void ReleaseRHI() override
	{
		FTextureResource::ReleaseRHI();
		RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
	}
	
	virtual uint32 GetSizeX() const override
	{
		return SizeX;
	}
	
	virtual uint32 GetSizeY() const override
	{
		return SizeY;
	}
	
	void UpdateDepthData_RenderThread(FRHICommandListImmediate& RHICmdList, const float* DepthData, int32 ImageWidth, int32 ImageHeight)
	{
		if (SizeX != ImageWidth || SizeY != ImageHeight)
		{
			SizeX = ImageWidth;
			SizeY = ImageHeight;
			
			// Re-initialize the texture if the size changes
			InitRHI(RHICmdList);
		}
		
		if (TextureRHI)
		{
			auto Texture2D = (FRHITexture*)(TextureRHI.GetReference());
			FUpdateTextureRegion2D Region(0, 0, 0, 0, ImageWidth, ImageHeight);
			RHICmdList.UpdateTexture2D(Texture2D, 0, Region, sizeof(float) * ImageWidth, (const uint8*)DepthData);
		}
	}
	
	void UpdateDepthData(const uint16* DepthData, int32 DepthDataSize, int32 ImageWidth, int32 ImageHeight)
	{
		const auto NumPixels = ImageWidth * ImageHeight;
		float* ConvertedDepthData = (float*)FMemory::Malloc(sizeof(float) * NumPixels);
		for (auto Index = 0; Index < NumPixels; ++Index)
		{
			ConvertedDepthData[Index] = DepthData[Index];
		}
		
		ENQUEUE_RENDER_COMMAND(UpdateDepthTexture)([this, ConvertedDepthData, ImageWidth, ImageHeight](FRHICommandListImmediate& RHICmdList)
		{
			UpdateDepthData_RenderThread(RHICmdList, ConvertedDepthData, ImageWidth, ImageHeight);
			FMemory::Free(ConvertedDepthData);
		});
	}
	
private:
	UARCoreDepthTexture* Owner = nullptr;
	uint32 SizeX = 0;
	uint32 SizeY = 0;
};

UARCoreDepthTexture::UARCoreDepthTexture(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	SRGB = false;
}

FTextureResource* UARCoreDepthTexture::CreateResource()
{
	return new FARCoreDepthTextureResource(this, Size.X, Size.Y);
}

#if PLATFORM_ANDROID
void UARCoreDepthTexture::UpdateDepthImage(const ArSession* SessionHandle, const ArImage* DepthImage)
{
	if (!DepthImage)
	{
		return;
	}
	
	ArImageFormat ImageFormat = AR_IMAGE_FORMAT_INVALID;
	ArImage_getFormat(SessionHandle, DepthImage, &ImageFormat);
	if (ImageFormat != AR_IMAGE_FORMAT_DEPTH16)
	{
		return;
	}
	
	const uint8_t* DepthData = nullptr;
	auto PlaneSize = 0;
	ArImage_getPlaneData(SessionHandle, DepthImage, 0, &DepthData, &PlaneSize);
	if (!DepthData)
	{
		return;
	}
	
	auto ImageWidth = 0;
	auto ImageHeight = 0;
	ArImage_getWidth(SessionHandle, DepthImage, &ImageWidth);
    ArImage_getHeight(SessionHandle, DepthImage, &ImageHeight);
	
	Size.X = ImageWidth;
	Size.Y = ImageHeight;
	
	if (!GetResource())
	{
		UpdateResource();
	}
	
	if (auto MyResource = static_cast<FARCoreDepthTextureResource*>(GetResource()))
	{
		MyResource->UpdateDepthData((const uint16*)DepthData, PlaneSize, ImageWidth, ImageHeight);
	}
}
#endif
