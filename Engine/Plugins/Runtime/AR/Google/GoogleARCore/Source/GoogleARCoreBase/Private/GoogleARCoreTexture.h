// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GoogleARCoreYCbCrConversion.h"
#include "ARTextures.h"
#include "Misc/Guid.h"

#if PLATFORM_ANDROID
#include "arcore_c_api.h"
#endif

#include "GoogleARCoreTexture.generated.h"


UCLASS()
class GOOGLEARCOREBASE_API UARCoreCameraTexture : public UARTexture
												, public IYCbCrConversionQuery
{
	GENERATED_UCLASS_BODY()
	
public:
	// UTexture interface implementation
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_TextureExternal; }
	// End UTexture interface
	
	uint32 GetTextureId() const;

	// Recreates image resource using Vulkan's external Hardware Buffer.
	void UpdateCameraImage(void* InVulkanHardwareBuffer);

	// IYCbCrConversionQuery overrides
	FYCbCrConversion GetYCbCrConversion_RenderThread() const override;

private:
	void* VukanHardwareBuffer = nullptr;
};


UCLASS()
class GOOGLEARCOREBASE_API UARCoreDepthTexture : public UARTexture
{
	GENERATED_UCLASS_BODY()
	
public:
	// UTexture interface implementation
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_Texture2D; }
	// End UTexture interface
	
#if PLATFORM_ANDROID
	void UpdateDepthImage(const ArSession* SessionHandle, const ArImage* Image);
#endif
};
