// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreXRCamera.h"
#include "GoogleARCoreXRTrackingSystem.h"
#include "SceneView.h"
#include "GoogleARCorePassthroughCameraRenderer.h"
#include "GoogleARCoreAndroidHelper.h"
#include "RenderGraphUtils.h"
#include "RenderUtils.h"

#if PLATFORM_ANDROID
#include <GLES2/gl2.h>
#endif

FGoogleARCoreXRCamera::FGoogleARCoreXRCamera(const FAutoRegister& AutoRegister, FGoogleARCoreXRTrackingSystem& InARCoreSystem, int32 InDeviceID)
	: FDefaultXRCamera(AutoRegister, &InARCoreSystem, InDeviceID)
	, GoogleARCoreTrackingSystem(InARCoreSystem)
	, bMatchDeviceCameraFOV(false)
	, bEnablePassthroughCameraRendering_RT(false)
{
	PassthroughRenderer = new FGoogleARCorePassthroughCameraRenderer();
}

void FGoogleARCoreXRCamera::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	FDefaultXRCamera::SetupView(InViewFamily, InView);
}

void FGoogleARCoreXRCamera::SetupViewProjectionMatrix(FSceneViewProjectionData& InOutProjectionData)
{
	if (GoogleARCoreTrackingSystem.ARCoreDeviceInstance->GetIsARCoreSessionRunning() && bMatchDeviceCameraFOV)
	{
		FIntRect ViewRect = InOutProjectionData.GetViewRect();
		InOutProjectionData.ProjectionMatrix = GoogleARCoreTrackingSystem.ARCoreDeviceInstance->GetPassthroughCameraProjectionMatrix(ViewRect.Size());
	}
}

void FGoogleARCoreXRCamera::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	FDefaultXRCamera::BeginRenderViewFamily(InViewFamily);
}

void FGoogleARCoreXRCamera::PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	FDefaultXRCamera::PreRenderViewFamily_RenderThread(GraphBuilder, InViewFamily);

	FGoogleARCoreXRTrackingSystem& TS = GoogleARCoreTrackingSystem;

	if (TS.ARCoreDeviceInstance->GetIsARCoreSessionRunning() && bEnablePassthroughCameraRendering_RT)
	{
		PassthroughRenderer->InitializeRenderer_RenderThread(InViewFamily);
	}

#if PLATFORM_ANDROID
	// Culling inversion ignored in Vulkan. Investigate when it's required and approach to apply it in Vulkan.
	if (!FAndroidMisc::ShouldUseVulkan())
	{
		if(TS.ARCoreDeviceInstance->GetIsARCoreSessionRunning() && TS.ARCoreDeviceInstance->GetShouldInvertCulling())
		{
			glFrontFace(GL_CW);
		}
		else
		{
			glFrontFace(GL_CCW);
		}
	}
#endif
}

BEGIN_SHADER_PARAMETER_STRUCT(FPostBasePassViewExtensionParameters, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FGoogleARCoreXRCamera::PostRenderBasePassDeferred_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView, const FRenderTargetBindingSlots& RenderTargets, TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FPostBasePassViewExtensionParameters>();
	PassParameters->RenderTargets = RenderTargets;
	PassParameters->SceneTextures = SceneTextures;

	GraphBuilder.AddPass(RDG_EVENT_NAME("RenderVideoOverlay_RenderThread"), PassParameters, ERDGPassFlags::Raster, [this, &InView](FRHICommandListImmediate& RHICmdList)
	{
		PassthroughRenderer->RenderVideoOverlay_RenderThread(RHICmdList, InView);
	});
}

void FGoogleARCoreXRCamera::PostRenderBasePassMobile_RenderThread(FRHICommandList& RHICmdList, FSceneView& InView)
{
	PassthroughRenderer->RenderVideoOverlay_RenderThread(RHICmdList, InView);
}

bool FGoogleARCoreXRCamera::GetPassthroughCameraUVs_RenderThread(TArray<FVector2D>& OutUVs)
{
	if (GoogleARCoreTrackingSystem.ARCoreDeviceInstance->GetIsARCoreSessionRunning() 
		&& bEnablePassthroughCameraRendering_RT && GoogleARCoreTrackingSystem.ARCoreDeviceInstance->GetPassthroughCameraTimestamp() != 0)
	{
		TArray<float> TransformedUVs;
		// TODO save the transformed UVs and only calculate if uninitialized or FGoogleARCoreFrame::IsDisplayRotationChanged() returns true
		static const TArray<float> OverlayQuadUVs = { 0.0f, 0.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f };
		GoogleARCoreTrackingSystem.ARCoreDeviceInstance->GetPassthroughCameraImageUVs(OverlayQuadUVs, TransformedUVs);
		
		OutUVs.SetNumUninitialized(4);

		bool bFlipCameraImageVertically = !IsMobileHDR();
		if (bFlipCameraImageVertically)
		{
			OutUVs[1] = FVector2D(TransformedUVs[0], TransformedUVs[1]);
			OutUVs[0] = FVector2D(TransformedUVs[2], TransformedUVs[3]);
			OutUVs[3] = FVector2D(TransformedUVs[4], TransformedUVs[5]);
			OutUVs[2] = FVector2D(TransformedUVs[6], TransformedUVs[7]);
		}
		else
		{
			OutUVs[0] = FVector2D(TransformedUVs[0], TransformedUVs[1]);
			OutUVs[1] = FVector2D(TransformedUVs[2], TransformedUVs[3]);
			OutUVs[2] = FVector2D(TransformedUVs[4], TransformedUVs[5]);
			OutUVs[3] = FVector2D(TransformedUVs[6], TransformedUVs[7]);
		}
		return true;
	}
	else
	{
		return false;
	}
}

bool FGoogleARCoreXRCamera::IsActiveThisFrame_Internal(const FSceneViewExtensionContext&) const
{
	return GoogleARCoreTrackingSystem.IsHeadTrackingAllowed();
}

void FGoogleARCoreXRCamera::ConfigXRCamera(bool bInMatchDeviceCameraFOV, bool bInEnablePassthroughCameraRendering)
{
	bMatchDeviceCameraFOV = bInMatchDeviceCameraFOV;
	FGoogleARCoreXRCamera* ARCoreXRCamera = this;
	ENQUEUE_RENDER_COMMAND(ConfigXRCamera)(
		[ARCoreXRCamera, bInEnablePassthroughCameraRendering](FRHICommandListImmediate& RHICmdList)
		{
			ARCoreXRCamera->bEnablePassthroughCameraRendering_RT = bInEnablePassthroughCameraRendering;
		}
	);
}

void FGoogleARCoreXRCamera::UpdateCameraTextures(UTexture* NewCameraTexture, UTexture* DepthTexture, bool bEnableOcclusion)
{
	if (PassthroughRenderer)
	{
		PassthroughRenderer->UpdateCameraTextures(NewCameraTexture, DepthTexture, bEnableOcclusion);
	}
}

void FGoogleARCoreXRCamera::UpdateCameraYCbCrConversion(IYCbCrConversionQuery* NewYCbCrConversionQuery)
{
	if (PassthroughRenderer)
	{
		PassthroughRenderer->UpdateCameraYCbCrConversion(NewYCbCrConversionQuery);
	}
}
