// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DefaultXRCamera.h"

class FGoogleARCoreXRTrackingSystem;
class FGoogleARCorePassthroughCameraRenderer;
class UTexture;
class IYCbCrConversionQuery;

class FGoogleARCoreXRCamera : public FDefaultXRCamera
{
public:
	FGoogleARCoreXRCamera(const FAutoRegister&, FGoogleARCoreXRTrackingSystem& InARCoreSystem, int32 InDeviceID);

	//~ FDefaultXRCamera
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual void SetupViewProjectionMatrix(FSceneViewProjectionData& InOutProjectionData) override;
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	virtual void PostRenderBasePassDeferred_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView, const FRenderTargetBindingSlots& RenderTargets, TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures) override;
	virtual void PostRenderBasePassMobile_RenderThread(FRHICommandList& RHICmdList, FSceneView& InView) override;
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
	virtual bool GetPassthroughCameraUVs_RenderThread(TArray<FVector2D>& OutUVs) override;
	//~ FDefaultXRCamera

	void ConfigXRCamera(bool bInMatchDeviceCameraFOV, bool bInEnablePassthroughCameraRendering);
	
	void UpdateCameraTextures(UTexture* NewCameraTexture, UTexture* DepthTexture, bool bEnableOcclusion);

	void UpdateCameraYCbCrConversion(IYCbCrConversionQuery* NewYCbCrConversionQuery);

private:
	FGoogleARCoreXRTrackingSystem& GoogleARCoreTrackingSystem;
	FGoogleARCorePassthroughCameraRenderer* PassthroughRenderer;

	bool bMatchDeviceCameraFOV;
	bool bEnablePassthroughCameraRendering_RT;
};
