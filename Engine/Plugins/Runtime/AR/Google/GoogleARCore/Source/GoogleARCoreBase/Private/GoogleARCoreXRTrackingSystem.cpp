// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreXRTrackingSystem.h"
#include "Engine/Engine.h"
#include "Misc/EngineVersion.h"
#include "RHIDefinitions.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/WorldSettings.h"
#include "GoogleARCoreDevice.h"
#include "GoogleARCoreTypes.h"
#include "GoogleARCoreXRCamera.h"
#include "ARSessionConfig.h"
#include "GoogleARCoreBaseModule.h"
#include "GoogleARCoreDependencyHandler.h"
#include "TextureResource.h"


DECLARE_CYCLE_STAT(TEXT("OnStartGameFrame"), STAT_OnStartGameFrame, STATGROUP_ARCore);

static const FName ARCoreSystemName(TEXT("FGoogleARCoreXRTrackingSystem"));

FGoogleARCoreXRTrackingSystem* FGoogleARCoreXRTrackingSystem::GetInstance()
{
	if (GEngine->XRSystem.IsValid() && (GEngine->XRSystem->GetSystemName() == ARCoreSystemName))
	{
		return static_cast<FGoogleARCoreXRTrackingSystem*>(GEngine->XRSystem.Get());
	}
	
	return nullptr;
}

FGoogleARCoreXRTrackingSystem::FGoogleARCoreXRTrackingSystem()
	: FXRTrackingSystemBase(this)
	, ARCoreDeviceInstance(nullptr)
	, bMatchDeviceCameraFOV(false)
	, bEnablePassthroughCameraRendering(false)
	, bHasValidPose(false)
	, CachedPosition(FVector::ZeroVector)
	, CachedOrientation(FQuat::Identity)
	, DeltaControlRotation(FRotator::ZeroRotator)
	, DeltaControlOrientation(FQuat::Identity)
	, LightEstimate(nullptr)
	, EventManager(nullptr)
{
	UE_LOGF(LogGoogleARCoreTrackingSystem, Log, "Creating GoogleARCore Tracking System.");
	ARCoreDeviceInstance = FGoogleARCoreDevice::GetInstance();
	check(ARCoreDeviceInstance);
	
	IModularFeatures::Get().RegisterModularFeature(UGoogleARCoreDependencyHandler::GetModularFeatureName(), GetMutableDefault<UGoogleARCoreDependencyHandler>());
}

FGoogleARCoreXRTrackingSystem::~FGoogleARCoreXRTrackingSystem()
{
	IModularFeatures::Get().UnregisterModularFeature(UGoogleARCoreDependencyHandler::GetModularFeatureName(), GetMutableDefault<UGoogleARCoreDependencyHandler>());
}

/////////////////////////////////////////////////////////////////////////////////
// Begin FGoogleARCoreXRTrackingSystem IHeadMountedDisplay Virtual Interface   //
////////////////////////////////////////////////////////////////////////////////
FName FGoogleARCoreXRTrackingSystem::GetSystemName() const
{
	return ARCoreSystemName;
}

int32 FGoogleARCoreXRTrackingSystem::GetXRSystemFlags() const
{
	return EXRSystemFlags::IsAR | EXRSystemFlags::IsTablet;
}

bool FGoogleARCoreXRTrackingSystem::IsHeadTrackingAllowed() const
{
#if PLATFORM_ANDROID
	return FGoogleARCoreDevice::GetInstance()->GetSessionStatus().Status == EARSessionStatus::Running &&
		FGoogleARCoreDevice::GetInstance()->GetARSystem()->GetSessionConfig().ShouldEnableCameraTracking();
#else
	return false;
#endif
}

bool FGoogleARCoreXRTrackingSystem::GetCurrentPose(int32 DeviceId, FQuat& OutOrientation, FVector& OutPosition)
{
	if (DeviceId == IXRTrackingSystem::HMDDeviceId && ARCoreDeviceInstance->GetIsARCoreSessionRunning())
	{
		OutOrientation = CachedOrientation;
		OutPosition = CachedPosition;
		return true;
	}
	else
	{
		return false;
	}
}

FString FGoogleARCoreXRTrackingSystem::GetVersionString() const
{
	FString s = FString::Printf(TEXT("ARCoreHMD - %s, built %s, %s"), *FEngineVersion::Current().ToString(),
		UTF8_TO_TCHAR(__DATE__), UTF8_TO_TCHAR(__TIME__));

	return s;
}

bool FGoogleARCoreXRTrackingSystem::EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type /*= EXRTrackedDeviceType::Any*/)
{
	if (Type == EXRTrackedDeviceType::Any || Type == EXRTrackedDeviceType::HeadMountedDisplay)
	{
		static const int32 DeviceId = IXRTrackingSystem::HMDDeviceId;
		OutDevices.Add(DeviceId);
		return true;
	}
	return false;
}

bool FGoogleARCoreXRTrackingSystem::OnStartGameFrame(FWorldContext& WorldContext)
{
	SCOPE_CYCLE_COUNTER(STAT_OnStartGameFrame);
	
	if (auto World = WorldContext.World())
	{
		ARCoreDeviceInstance->UpdateGameFrame(World);
	}
	
	FTransform CurrentPose;
	bHasValidPose = false;
	if (ARCoreDeviceInstance->GetIsARCoreSessionRunning())
	{
		if (ARCoreDeviceInstance->GetTrackingState() == EGoogleARCoreTrackingState::Tracking || ARCoreDeviceInstance->GetIsFrontCameraSession())
		{
			CurrentPose = ARCoreDeviceInstance->GetLatestPose();
			CurrentPose *= GetARCompositionComponent()->GetAlignmentTransform();
			bHasValidPose = true;
			RefreshTrackingToWorldTransform(WorldContext);
		}

		if (bHasValidPose)
		{
			CachedOrientation = CurrentPose.GetRotation();
			CachedPosition = CurrentPose.GetTranslation();
		}

		if (LightEstimate == nullptr)
		{
			LightEstimate = NewObject<UARBasicLightEstimate>();
		}
		FGoogleARCoreLightEstimate ARCoreLightEstimate = FGoogleARCoreDevice::GetInstance()->GetLatestLightEstimate();
		if (ARCoreLightEstimate.bIsValid)
		{
			LightEstimate->SetLightEstimate(ARCoreLightEstimate.RGBScaleFactor, ARCoreLightEstimate.PixelIntensity);
		}
		else
		{
			LightEstimate = nullptr;
		}
		
		if (auto Camera = GetARCoreCamera())
		{
			auto CameraYCbCrConversionQuery = ARCoreDeviceInstance->GetLastCameraYCbCrConversionQuery();
			Camera->UpdateCameraYCbCrConversion(CameraYCbCrConversionQuery);

			auto CameraTexture = ARCoreDeviceInstance->GetLastCameraTexture();
			auto DepthTexture = ARCoreDeviceInstance->GetDepthTexture();
			Camera->UpdateCameraTextures(CameraTexture, DepthTexture, bWantsDepthOcclusion);
		}
	}

	return true;
}

FGoogleARCoreXRCamera* FGoogleARCoreXRTrackingSystem::GetARCoreCamera()
{
	return static_cast<FGoogleARCoreXRCamera*>(GetXRCamera().Get());
}

void FGoogleARCoreXRTrackingSystem::ConfigARCoreXRCamera(bool bInMatchDeviceCameraFOV, bool bInEnablePassthroughCameraRendering)
{
	bMatchDeviceCameraFOV = bInMatchDeviceCameraFOV;
	bEnablePassthroughCameraRendering = bInEnablePassthroughCameraRendering;

	if (auto Camera = GetARCoreCamera())
	{
		Camera->ConfigXRCamera(bEnablePassthroughCameraRendering, bEnablePassthroughCameraRendering);
	}
}


void FGoogleARCoreXRTrackingSystem::EnableColorCameraRendering(bool bInEnablePassthroughCameraRendering)
{
	bEnablePassthroughCameraRendering = bInEnablePassthroughCameraRendering;
	if (auto Camera = GetARCoreCamera())
	{
		Camera->ConfigXRCamera(bEnablePassthroughCameraRendering, bEnablePassthroughCameraRendering);
	}
}

bool FGoogleARCoreXRTrackingSystem::GetColorCameraRenderingEnabled()
{
	return bEnablePassthroughCameraRendering;
}

float FGoogleARCoreXRTrackingSystem::GetWorldToMetersScale() const
{
	if (IsInGameThread() && GWorld != nullptr)
	{
		return GWorld->GetWorldSettings()->WorldToMeters;
	}

	// Default value, assume Unreal units are in centimeters
	return 100.0f;
}

void* FGoogleARCoreXRTrackingSystem::GetARSessionRawPointer()
{
#if PLATFORM_ANDROID
	return static_cast<void*>(FGoogleARCoreDevice::GetInstance()->GetARSessionRawPointer());
#endif
	ensureAlwaysMsgf(false, TEXT("FGoogleARCoreXRTrackingSystem::GetARSessionRawPointer is unimplemented on current platform."));
	return nullptr;
}

void* FGoogleARCoreXRTrackingSystem::GetGameThreadARFrameRawPointer()
{
#if PLATFORM_ANDROID
	return static_cast<void*>(FGoogleARCoreDevice::GetInstance()->GetGameThreadARFrameRawPointer());
#endif
	ensureAlwaysMsgf(false, TEXT("FGoogleARCoreXRTrackingSystem::GetARSessionRawPointer is unimplemented on current platform."));
	return nullptr;
}

UGoogleARCoreEventManager* FGoogleARCoreXRTrackingSystem::GetEventManager()
{
	if (EventManager == nullptr)
	{
		EventManager = NewObject<UGoogleARCoreEventManager>();
	}
	return EventManager;
}

void FGoogleARCoreXRTrackingSystem::OnARSystemInitialized()
{

}

EARTrackingQuality FGoogleARCoreXRTrackingSystem::OnGetTrackingQuality() const
{
	if (!bHasValidPose)
	{
		return EARTrackingQuality::NotTracking;
	}

	return EARTrackingQuality::OrientationAndPosition;
}


EARTrackingQualityReason FGoogleARCoreXRTrackingSystem::OnGetTrackingQualityReason() const
{
	EGoogleARCoreTrackingFailureReason GoogleARFailureReason = FGoogleARCoreDevice::GetInstance()->GetTrackingFailureReason();

	// Dont return EARTrackingQualityReason::None, which means that the tracking quality is not limited
	EARTrackingQualityReason TrackingQualityReason;

	// tracking quality reason
	switch (GoogleARFailureReason)
	{
	case EGoogleARCoreTrackingFailureReason::None:
		TrackingQualityReason = EARTrackingQualityReason::None;
		break;

	case EGoogleARCoreTrackingFailureReason::BadState:
		TrackingQualityReason = EARTrackingQualityReason::BadState;
		break;

	case EGoogleARCoreTrackingFailureReason::InsufficientLight:
		TrackingQualityReason = EARTrackingQualityReason::InsufficientLight;
		break;

	case EGoogleARCoreTrackingFailureReason::ExcessiveMotion:
		TrackingQualityReason = EARTrackingQualityReason::ExcessiveMotion;
		break;

	default:
		case EGoogleARCoreTrackingFailureReason::InsufficientFeatures:
		TrackingQualityReason = EARTrackingQualityReason::InsufficientFeatures;
		break;
	}

	return TrackingQualityReason;
}

bool FGoogleARCoreXRTrackingSystem::IsARAvailable() const
{
	return FGoogleARCoreDevice::GetInstance()->CheckARCoreAPKAvailability() == EGoogleARCoreAvailability::SupportedInstalled;
}


void FGoogleARCoreXRTrackingSystem::OnStartARSession(UARSessionConfig* SessionConfig)
{
	FGoogleARCoreDevice::GetInstance()->StartARCoreSessionRequest(SessionConfig);
	bWantsDepthOcclusion = SessionConfig->bUseSceneDepthForOcclusion;
}

void FGoogleARCoreXRTrackingSystem::OnPauseARSession()
{
	FGoogleARCoreDevice::GetInstance()->PauseARCoreSession();
}

void FGoogleARCoreXRTrackingSystem::OnStopARSession()
{
	FGoogleARCoreDevice::GetInstance()->PauseARCoreSession();
	FGoogleARCoreDevice::GetInstance()->ResetARCoreSession();
}

FARSessionStatus FGoogleARCoreXRTrackingSystem::OnGetARSessionStatus() const
{
	return FGoogleARCoreDevice::GetInstance()->GetSessionStatus();
}

void FGoogleARCoreXRTrackingSystem::OnSetAlignmentTransform(const FTransform& InAlignmentTransform)
{
	const FTransform& NewAlignmentTransform = InAlignmentTransform;

	TArray<UARTrackedGeometry*> AllTrackedGeometries = GetARCompositionComponent()->GetAllTrackedGeometries();
	for (UARTrackedGeometry* TrackedGeometry : AllTrackedGeometries)
	{
		TrackedGeometry->UpdateAlignmentTransform(NewAlignmentTransform);
	}

	TArray<UARPin*> AllARPins = GetARCompositionComponent()->GetAllPins();
	for (UARPin* SomePin : AllARPins)
	{
		SomePin->UpdateAlignmentTransform(NewAlignmentTransform);
	}
}

static EGoogleARCoreLineTraceChannel ConvertToGoogleARCoreTraceChannels(EARLineTraceChannels TraceChannels)
{
	EGoogleARCoreLineTraceChannel ARCoreTraceChannels = EGoogleARCoreLineTraceChannel::None;
	if (!!(TraceChannels & EARLineTraceChannels::FeaturePoint))
	{
		ARCoreTraceChannels = ARCoreTraceChannels | EGoogleARCoreLineTraceChannel::FeaturePoint;
	}
	if (!!(TraceChannels & EARLineTraceChannels::GroundPlane))
	{
		ARCoreTraceChannels = ARCoreTraceChannels | EGoogleARCoreLineTraceChannel::InfinitePlane;
	}
	if (!!(TraceChannels & EARLineTraceChannels::PlaneUsingBoundaryPolygon))
	{
		ARCoreTraceChannels = ARCoreTraceChannels | EGoogleARCoreLineTraceChannel::PlaneUsingBoundaryPolygon;
	}
	if (!!(TraceChannels & EARLineTraceChannels::PlaneUsingExtent))
	{
		ARCoreTraceChannels = ARCoreTraceChannels | EGoogleARCoreLineTraceChannel::PlaneUsingExtent;
	}
	return ARCoreTraceChannels;
}

TArray<FARTraceResult> FGoogleARCoreXRTrackingSystem::OnLineTraceTrackedObjects(const FVector2D ScreenCoord, EARLineTraceChannels TraceChannels)
{
	TArray<FARTraceResult> OutHitResults;
	FGoogleARCoreDevice::GetInstance()->ARLineTrace(ScreenCoord, ConvertToGoogleARCoreTraceChannels(TraceChannels), OutHitResults);
	return OutHitResults;
}

TArray<FARTraceResult> FGoogleARCoreXRTrackingSystem::OnLineTraceTrackedObjects(const FVector Start, const FVector End, EARLineTraceChannels TraceChannels)
{
	TArray<FARTraceResult> OutHitResults;
	FGoogleARCoreDevice::GetInstance()->ARLineTrace(Start, End, ConvertToGoogleARCoreTraceChannels(TraceChannels), OutHitResults);
	return OutHitResults;
}

TArray<UARTrackedGeometry*> FGoogleARCoreXRTrackingSystem::OnGetAllTrackedGeometries() const
{
	TArray<UARTrackedGeometry*> AllTrackedGeometry;
	FGoogleARCoreDevice::GetInstance()->GetAllTrackables<UARTrackedGeometry>(AllTrackedGeometry);
	return AllTrackedGeometry;
}

TArray<UARPin*> FGoogleARCoreXRTrackingSystem::OnGetAllPins() const
{
	TArray<UARPin*> AllARPins;
	FGoogleARCoreDevice::GetInstance()->GetAllARPins(AllARPins);
	return AllARPins;
}

bool FGoogleARCoreXRTrackingSystem::OnIsTrackingTypeSupported(EARSessionType SessionType) const
{
	return FGoogleARCoreDevice::GetInstance()->GetIsTrackingTypeSupported(SessionType);
}

UARLightEstimate* FGoogleARCoreXRTrackingSystem::OnGetCurrentLightEstimate() const
{
	return LightEstimate;
}

UARPin* FGoogleARCoreXRTrackingSystem::FindARPinByComponent(const USceneComponent* Component) const
{
	TArray<UARPin*> Pins;
	FGoogleARCoreDevice::GetInstance()->GetAllARPins(Pins);
	for (UARPin* Pin : Pins)
	{
		if (Pin->GetPinnedComponent() == Component)
		{
			return Pin;
		}
	}

	return nullptr;
}

UARPin* FGoogleARCoreXRTrackingSystem::OnPinComponent(USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry /*= nullptr*/, const FName DebugName /*= NAME_None*/)
{
	UARPin* NewARPin = nullptr;
	// TODO: error handling?
	FGoogleARCoreDevice::GetInstance()->CreateARPin(PinToWorldTransform, TrackedGeometry, ComponentToPin, DebugName, NewARPin);
	return NewARPin;
}

void FGoogleARCoreXRTrackingSystem::OnRemovePin(UARPin* PinToRemove)
{
	FGoogleARCoreDevice::GetInstance()->RemoveARPin(PinToRemove);
}

bool FGoogleARCoreXRTrackingSystem::OnTryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InPinName, UARPin*& OutPin)
{
	return FGoogleARCoreDevice::GetInstance()->TryGetOrCreatePinForNativeResource(InNativeResource, InPinName, OutPin);
}

static TSharedPtr<FGoogleARCoreSession> CreateTempARCoreSession(EARSessionType SessionType)
{
	if (SessionType == EARSessionType::None)
	{
		return nullptr;
	}
	
	return FGoogleARCoreSession::CreateARCoreSession();
}

TArray<FARVideoFormat> FGoogleARCoreXRTrackingSystem::OnGetSupportedVideoFormats(EARSessionType SessionType) const
{
	if (SessionType == EARSessionType::None)
	{
		return {};
	}

	// We're creating a temp session here as the "current" session from FGoogleARCoreDevice may not be available
	// or it may use a different session type
	auto NewARCoreSession = CreateTempARCoreSession(SessionType);
	if (NewARCoreSession && NewARCoreSession->GetSessionCreateStatus() == EGoogleARCoreAPIStatus::AR_SUCCESS)
	{
		TArray<FGoogleARCoreCameraConfig> SupportedCameraConfig = NewARCoreSession->GetSupportedCameraConfig(SessionType == EARSessionType::Face ? EGoogleARCoreCameraFacing::Back : EGoogleARCoreCameraFacing::Front);

		TArray<FARVideoFormat> VideoFormats;

		for (const FGoogleARCoreCameraConfig& CameraConfig : SupportedCameraConfig) //-V1078  // Array always empty on non-android, but this code only actually runs on android.
		{
			// FPS is not exposed so we assume it's 30...
			// Note that we're using CameraTextureResolution rather than CameraImageResolution as the former is relevant
			// to the camera passthrough rendering
			VideoFormats.AddUnique({ CameraConfig.GetMaxFPS(), CameraConfig.CameraTextureResolution.X, CameraConfig.CameraTextureResolution.Y });
		}

		return VideoFormats;
	}

	return {};
}

TArray<FVector> FGoogleARCoreXRTrackingSystem::OnGetPointCloud() const
{
	TArray<FVector> PointCloudPoints;
	UGoogleARCorePointCloud* LatestPointCloud = nullptr;
	if (!(FGoogleARCoreDevice::GetInstance()->GetLatestPointCloud(LatestPointCloud) == EGoogleARCoreFunctionStatus::Success))
	{
		return PointCloudPoints;
	}

#if PLATFORM_ANDROID //Static analysis complains that i is always < 0, because GetPointNum returns 0 on non-android platforms
	for(int i=0; i<LatestPointCloud->GetPointNum(); i++)
	{
		FVector Point = FVector::ZeroVector;
		float Confident = 0;
		LatestPointCloud->GetPoint(i, Point, Confident);
		PointCloudPoints.Add(Point);
	}
#endif

	return PointCloudPoints;
}

bool FGoogleARCoreXRTrackingSystem::OnAddRuntimeCandidateImage(UARSessionConfig* SessionConfig, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth)
{
	EPixelFormat PixelFormat = CandidateTexture->GetPixelFormat();

	if (PixelFormat == EPixelFormat::PF_B8G8R8A8 || PixelFormat == EPixelFormat::PF_G8)
	{
		ensure(CandidateTexture->GetNumMips() > 0);
		FTexture2DMipMap* Mip0 = &CandidateTexture->GetPlatformData()->Mips[0];
		FByteBulkData* RawImageData = &Mip0->BulkData;

		int ImageWidth = CandidateTexture->GetSizeX();
		int ImageHeight = CandidateTexture->GetSizeY();

		TArray<uint8> GrayscaleBuffer;
		int PixelNum = ImageWidth * ImageHeight;
		uint8* RawBytes = static_cast<uint8*>(RawImageData->Lock(LOCK_READ_ONLY));
		if (PixelFormat == EPixelFormat::PF_B8G8R8A8)
		{
			GrayscaleBuffer.SetNumUninitialized(PixelNum);
			ensureMsgf(RawImageData->GetBulkDataSize() == ImageWidth * ImageHeight * 4,
				TEXT("Unsupported texture data when adding runtime candidate image."));
			for (int i = 0; i < PixelNum; i++)
			{
				uint8 R = RawBytes[i * 4 + 2];
				uint8 G = RawBytes[i * 4 + 1];
				uint8 B = RawBytes[i * 4];
				GrayscaleBuffer[i] = 0.2126 * R + 0.7152 * G + 0.0722 * B;
			}
		}
		else
		{
			ensureMsgf(RawImageData->GetBulkDataSize() == ImageWidth * ImageHeight,
				TEXT("Unsupported texture data when adding runtime candidate image."));
			GrayscaleBuffer = TArray<uint8>(RawBytes, PixelNum);
		}
		RawImageData->Unlock();
		return AddRuntimeGrayscaleImage(SessionConfig, GrayscaleBuffer,
			ImageWidth, ImageHeight, FriendlyName, PhysicalWidth);
	}

	UE_LOGF(LogGoogleARCoreTrackingSystem, Warning, "Failed to add runtime candidate image: Unsupported texture format: %ls. ARCore only support PF_B8G8R8A8 or PF_G8 for now for adding runtime candidate image in ARCore", GetPixelFormatString(PixelFormat));
	return false;
}

bool FGoogleARCoreXRTrackingSystem::AddRuntimeGrayscaleImage(UARSessionConfig* SessionConfig, const TArray<uint8>& ImageGrayscalePixels, int ImageWidth, int ImageHeight, FString FriendlyName, float PhysicalWidth)
{
	return FGoogleARCoreDevice::GetInstance()->AddRuntimeCandidateImage(SessionConfig, ImageGrayscalePixels,
		ImageWidth, ImageHeight, FriendlyName, PhysicalWidth);
}

void FGoogleARCoreXRTrackingSystem::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (LightEstimate != nullptr)
	{
		Collector.AddReferencedObject(LightEstimate);
	}

	if (EventManager != nullptr)
	{
		Collector.AddReferencedObject(EventManager);
	}
}

TSharedPtr<class IXRCamera, ESPMode::ThreadSafe> FGoogleARCoreXRTrackingSystem::GetXRCamera(int32 DeviceId /*= HMDDeviceId*/)
{
	check(DeviceId == HMDDeviceId);

	// Don't create/load on the render thread
	if (!XRCamera.IsValid() && IsInGameThread())
	{
		XRCamera = FSceneViewExtensions::NewExtension<FGoogleARCoreXRCamera>(*this, DeviceId);
	}
	return XRCamera;
}

void FGoogleARCoreXRTrackingSystem::OnTrackableAdded(UARTrackedGeometry* InTrackedGeometry)
{
	if (ensure(InTrackedGeometry))
	{
		UE_LOGF(LogGoogleARCoreTrackingSystem, Log, "OnTrackableAdded: %ls %ls", *InTrackedGeometry->GetName(), *InTrackedGeometry->GetDebugName().ToString());
		TriggerOnTrackableAddedDelegates(InTrackedGeometry);
	}
}

void FGoogleARCoreXRTrackingSystem::OnTrackableUpdated(UARTrackedGeometry* InTrackedGeometry)
{
	if (ensure(InTrackedGeometry))
	{
		TriggerOnTrackableUpdatedDelegates(InTrackedGeometry);
	}
}

void FGoogleARCoreXRTrackingSystem::OnTrackableRemoved(UARTrackedGeometry* InTrackedGeometry)
{
	if (ensure(InTrackedGeometry))
	{
		UE_LOGF(LogGoogleARCoreTrackingSystem, Log, "OnTrackableRemoved: %ls %ls", *InTrackedGeometry->GetName(), *InTrackedGeometry->GetDebugName().ToString());
		TriggerOnTrackableRemovedDelegates(InTrackedGeometry);
	}
}

UARTexture* FGoogleARCoreXRTrackingSystem::OnGetARTexture(EARTextureType TextureType) const
{
	if (TextureType == EARTextureType::CameraImage)
	{
		return ARCoreDeviceInstance->GetLastCameraTexture();
	}
	else if (TextureType == EARTextureType::SceneDepthMap)
	{
		return ARCoreDeviceInstance->GetDepthTexture();
	}
	
	return nullptr;
}

bool FGoogleARCoreXRTrackingSystem::OnGetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const
{
	const auto Result = FGoogleARCoreDevice::GetInstance()->GetCameraTextureIntrinsics(OutCameraIntrinsics);
	return Result == EGoogleARCoreFunctionStatus::Success;
}

bool FGoogleARCoreXRTrackingSystem::OnIsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature) const
{
#if PLATFORM_ANDROID
	if (SessionTrackingFeature == EARSessionTrackingFeature::SceneDepth)
	{
		auto NewARCoreSession = CreateTempARCoreSession(SessionType);
		if (NewARCoreSession && NewARCoreSession->GetSessionCreateStatus() == EGoogleARCoreAPIStatus::AR_SUCCESS)
		{
			if (auto SessionHandle = NewARCoreSession->GetHandle())
			{
				int32_t bSupported = 0;
				ArSession_isDepthModeSupported(SessionHandle, AR_DEPTH_MODE_AUTOMATIC, &bSupported);
				return bSupported != 0;
			}
		}
		
	}
#endif
	
	return false;
}
