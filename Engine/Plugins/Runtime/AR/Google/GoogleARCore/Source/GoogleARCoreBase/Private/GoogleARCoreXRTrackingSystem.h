// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "XRTrackingSystemBase.h"
#include "HeadMountedDisplay.h"
#include "IHeadMountedDisplay.h"
#include "SceneViewExtension.h"
#include "Slate/SceneViewport.h"
#include "SceneView.h"
#include "GoogleARCoreDevice.h"
#include "ARSystemSupportBase.h"
#include "ARLightEstimate.h"

class UARTrackedGeometry;
class FGoogleARCoreXRCamera;

class FGoogleARCoreXRTrackingSystem : public FARSystemSupportBase, public FXRTrackingSystemBase, public FGCObject, public TSharedFromThis<FGoogleARCoreXRTrackingSystem, ESPMode::ThreadSafe>
{
	friend class FGoogleARCoreXRCamera;

public:
	static FGoogleARCoreXRTrackingSystem* GetInstance();
	
	FGoogleARCoreXRTrackingSystem();
	~FGoogleARCoreXRTrackingSystem();

	// FGoogleARCoreXRTrackingSystem Class Method
	void ConfigARCoreXRCamera(bool bMatchCameraFOV, bool bEnablePassthroughRendering);
	void EnableColorCameraRendering(bool bEnableColorCameraRnedering);
	bool GetColorCameraRenderingEnabled();

	// IXRTrackingSystem
	virtual FName GetSystemName() const override;
	virtual int32 GetXRSystemFlags() const override;
	virtual bool GetCurrentPose(int32 DeviceId, FQuat& OutOrientation, FVector& OutPosition) override;
	virtual FString GetVersionString() const override;
	virtual bool EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type = EXRTrackedDeviceType::Any) override;
	virtual TSharedPtr<class IXRCamera, ESPMode::ThreadSafe> GetXRCamera(int32 DeviceId = HMDDeviceId) override;
	// @todo : can I get rid of this? At least rename to IsCameraTracking / IsTrackingAllowed()
	virtual bool IsHeadTrackingAllowed() const override;
	virtual bool OnStartGameFrame(FWorldContext& WorldContext) override;
	// TODO: Figure out if we need to allow developer set/reset base orientation and position.
	virtual void ResetOrientationAndPosition(float yaw = 0.f) override {}
	// @todo move this to some interface
	virtual float GetWorldToMetersScale() const override;

	void* GetARSessionRawPointer() override;
	void* GetGameThreadARFrameRawPointer() override;

	UGoogleARCoreEventManager* GetEventManager();

	bool AddRuntimeGrayscaleImage(UARSessionConfig* SessionConfig, const TArray<uint8>& ImageGrayscalePixels, int ImageWidth, int ImageHeight,
		FString FriendlyName, float PhysicalWidth);
	
	void OnTrackableAdded(UARTrackedGeometry* InTrackedGeometry);
	void OnTrackableUpdated(UARTrackedGeometry* InTrackedGeometry);
	void OnTrackableRemoved(UARTrackedGeometry* InTrackedGeometry);

protected:
	// IARSystemSupport
	virtual bool IsARAvailable() const override;
	virtual void OnARSystemInitialized() override;
	virtual EARTrackingQuality OnGetTrackingQuality() const override;
	virtual EARTrackingQualityReason OnGetTrackingQualityReason() const override;
	virtual void OnStartARSession(UARSessionConfig* SessionConfig) override;
	virtual void OnPauseARSession() override;
	virtual void OnStopARSession() override;
	virtual FARSessionStatus OnGetARSessionStatus() const override;
	virtual void OnSetAlignmentTransform(const FTransform& InAlignmentTransform) override;
	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects(const FVector2D ScreenCoord, EARLineTraceChannels TraceChannels) override;
	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects(const FVector Start, const FVector End, EARLineTraceChannels TraceChannels) override;
	virtual TArray<UARTrackedGeometry*> OnGetAllTrackedGeometries() const override;
	virtual TArray<UARPin*> OnGetAllPins() const override;
	virtual bool OnIsTrackingTypeSupported(EARSessionType SessionType) const override;
	virtual UARLightEstimate* OnGetCurrentLightEstimate() const override;

	virtual UARPin* FindARPinByComponent(const USceneComponent* Component) const override;
	virtual UARPin* OnPinComponent(USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry = nullptr, const FName DebugName = NAME_None) override;
	virtual void OnRemovePin(UARPin* PinToRemove) override;
	virtual bool OnTryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InPinName, UARPin*& OutPin) override;

	virtual bool OnAddManualEnvironmentCaptureProbe(FVector Location, FVector Extent) override { return false; }
	virtual TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> OnGetCandidateObject(FVector Location, FVector Extent) const override { return TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe>(); }
	virtual TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> OnSaveWorld() const override { return TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe>(); }
// @todo -- support these properly
	virtual EARWorldMappingState OnGetWorldMappingStatus() const override { return EARWorldMappingState::StillMappingNotRelocalizable; }
	virtual TArray<FARVideoFormat> OnGetSupportedVideoFormats(EARSessionType SessionType) const override;
	virtual TArray<FVector> OnGetPointCloud() const override;
	virtual UARTexture* OnGetARTexture(EARTextureType TextureType) const override;
	bool OnGetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const override;
	bool OnIsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature) const override;
	//~IARSystemSupport

	virtual bool OnAddRuntimeCandidateImage(UARSessionConfig* SessionConfig, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth) override;
	
	FGoogleARCoreXRCamera* GetARCoreCamera();

private:
	//~ FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FGoogleARCoreXRTrackingSystem");
	}
	//~ FGCObject

private:
	FGoogleARCoreDevice* ARCoreDeviceInstance;

	bool bMatchDeviceCameraFOV;
	bool bEnablePassthroughCameraRendering;
	bool bHasValidPose;
	bool bWantsDepthOcclusion = false;

	FVector CachedPosition;
	FQuat CachedOrientation;
	FRotator DeltaControlRotation;    // same as DeltaControlOrientation but as rotator
	FQuat DeltaControlOrientation; // same as DeltaControlRotation but as quat

	TSharedPtr<class ISceneViewExtension, ESPMode::ThreadSafe> ViewExtension;

	TObjectPtr<UARBasicLightEstimate> LightEstimate;
	TObjectPtr<UGoogleARCoreEventManager> EventManager;
};

DEFINE_LOG_CATEGORY_STATIC(LogGoogleARCoreTrackingSystem, Log, All);
