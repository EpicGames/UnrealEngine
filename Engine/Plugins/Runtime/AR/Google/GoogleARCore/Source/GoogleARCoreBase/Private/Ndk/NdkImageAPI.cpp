// Copyright Epic Games, Inc. All Rights Reserved.

#include "NdkImageAPI.h"
#include "GoogleARCoreBaseLogCategory.h"

#if PLATFORM_ANDROID
#include <dlfcn.h>

static NdkImageAPI* NDKImageAPIInstance = nullptr;

NdkImageAPI* GetNdkImageAPI()
{
	if (NDKImageAPIInstance != nullptr)
	{
		return NDKImageAPIInstance;
	}

	NDKImageAPIInstance = new NdkImageAPI;

	void* LibHandle = dlopen("libmediandk.so", RTLD_NOW | RTLD_LOCAL);
	if(LibHandle == nullptr)
	{
		UE_LOGF(LogGoogleARCore, Error, "Failed to load libmediandk.so");
	}

	NDKImageAPIInstance->AImage_getHeight = (AImage_getHeight_ptr)dlsym(LibHandle, "AImage_getHeight");
	NDKImageAPIInstance->AImage_getNumberOfPlanes = (AImage_getNumberOfPlanes_ptr)dlsym(LibHandle, "AImage_getNumberOfPlanes");
	NDKImageAPIInstance->AImage_getPlaneData = (AImage_getPlaneData_ptr)dlsym(LibHandle, "AImage_getPlaneData");
	NDKImageAPIInstance->AImage_getPlanePixelStride = (AImage_getPlanePixelStride_ptr)dlsym(LibHandle, "AImage_getPlanePixelStride");
	NDKImageAPIInstance->AImage_getPlaneRowStride = (AImage_getPlaneRowStride_ptr)dlsym(LibHandle, "AImage_getPlaneRowStride");
	NDKImageAPIInstance->AImage_getWidth = (AImage_getWidth_ptr)dlsym(LibHandle, "AImage_getWidth");

	check(NDKImageAPIInstance->AImage_getHeight != nullptr);
	check(NDKImageAPIInstance->AImage_getNumberOfPlanes != nullptr);
	check(NDKImageAPIInstance->AImage_getPlaneData != nullptr);
	check(NDKImageAPIInstance->AImage_getPlanePixelStride != nullptr);
	check(NDKImageAPIInstance->AImage_getPlaneRowStride != nullptr);
	check(NDKImageAPIInstance->AImage_getWidth != nullptr);

	return NDKImageAPIInstance;
}
#endif