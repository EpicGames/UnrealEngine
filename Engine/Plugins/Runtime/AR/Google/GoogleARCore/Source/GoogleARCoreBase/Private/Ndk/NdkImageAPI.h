// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if PLATFORM_ANDROID

typedef struct AImage AImage;

// Function ptr;
typedef int32_t(*AImage_getWidth_ptr)(const AImage* image, /*out*/int32_t* width);
typedef int32_t(*AImage_getHeight_ptr)(const AImage* image, /*out*/int32_t* height);
typedef int32_t(*AImage_getNumberOfPlanes_ptr)(const AImage* image, /*out*/int32_t* numPlanes);
typedef int32_t(*AImage_getPlanePixelStride_ptr)(const AImage* image, int planeIdx, /*out*/int32_t* pixelStride);
typedef int32_t(*AImage_getPlaneRowStride_ptr)(const AImage* image, int planeIdx, /*out*/int32_t* rowStride);
typedef int32_t(*AImage_getPlaneData_ptr)(const AImage* image, int planeIdx, /*out*/uint8_t** data, /*out*/int* dataLength);

struct NdkImageAPI
{
	AImage_getWidth_ptr AImage_getWidth = nullptr;
	AImage_getHeight_ptr AImage_getHeight = nullptr;
	AImage_getNumberOfPlanes_ptr AImage_getNumberOfPlanes = nullptr;
	AImage_getPlanePixelStride_ptr AImage_getPlanePixelStride = nullptr;
	AImage_getPlaneRowStride_ptr AImage_getPlaneRowStride = nullptr;
	AImage_getPlaneData_ptr AImage_getPlaneData = nullptr;
};

NdkImageAPI* GetNdkImageAPI();

#define AImage_getWidth_dynamic GetNdkImageAPI()->AImage_getWidth
#define AImage_getHeight_dynamic GetNdkImageAPI()->AImage_getHeight
#define AImage_getNumberOfPlanes_dynamic GetNdkImageAPI()->AImage_getNumberOfPlanes
#define AImage_getPlanePixelStride_dynamic GetNdkImageAPI()->AImage_getPlanePixelStride
#define AImage_getPlaneRowStride_dynamic GetNdkImageAPI()->AImage_getPlaneRowStride
#define AImage_getPlaneData_dynamic GetNdkImageAPI()->AImage_getPlaneData

#endif