// Copyright Epic Games, Inc. All Rights Reserved.
#include "Misc/AutomationTest.h"

#include "GoogleARCoreAPI.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGoogleARCoreAPITest, "GoogleARCore.APITest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

void RunSessionCreateTest(FAutomationTestBase& Test)
{
	TSharedPtr<FGoogleARCoreSession> ARCoreSession = FGoogleARCoreSession::CreateARCoreSession();
	Test.TestNotNull("LatestFrame", ARCoreSession->GetLatestFrame());
	Test.TestNotNull("UObjectManager", ARCoreSession->GetUObjectManager());

	const FGoogleARCoreFrame* LatestFrame = ARCoreSession->GetLatestFrame();
	Test.TestEqual("LatestFrameTrackingState", LatestFrame->GetCameraTrackingState(), EGoogleARCoreTrackingState::StoppedTracking);
}

bool FGoogleARCoreAPITest::RunTest(const FString& Parameters)
{
	RunSessionCreateTest(*this);

	return true;
}
