// Copyright Epic Games, Inc. All Rights Reserved.
#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGoogleARCoreDummyTest, "GoogleARCore.DummyTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FGoogleARCoreDummyTest::RunTest(const FString& Parameters)
{
          return true;
}
