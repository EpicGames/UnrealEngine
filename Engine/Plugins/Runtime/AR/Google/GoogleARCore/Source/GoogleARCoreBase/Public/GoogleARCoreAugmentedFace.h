// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARTrackable.h"

#include "DrawDebugHelpers.h"

#include "GoogleARCoreAugmentedFace.generated.h"

/**
 * @ingroup GoogleARCoreBase
 * Describes the face regions for which the pose can be queried. Left and right
 * are defined relative to the actor (the person that the mesh belongs to).
 */
UENUM(BlueprintType)
enum class EGoogleARCoreAugmentedFaceRegion : uint8
{
	/* A region around the nose of the AugmentedFace. */
	NoseTip = 0,
	/* A region around the left forehead of the AugmentedFace. */
	ForeheadLeft = 1,
	/* A region around the right forehead of the AugmentedFace. */
	ForeheadRight = 2,
};

/**
 * An UObject representing a face detected by ARCore.
 */
UCLASS(BlueprintType)
class GOOGLEARCOREBASE_API UGoogleARCoreAugmentedFace : public UARFaceGeometry
{
	friend class FGoogleARCoreAugmentedFaceResource;

	GENERATED_BODY()

public:
	/**
	 * Draws the face mesh wireframe using debug lines.
	 */
	virtual void DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds = 0.0f) const override;

	/**
	 * Returns the latest known local-to-world transform of the given face region.
	 */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|AugmentedImage")
	FTransform GetLocalToWorldTransformOfRegion(EGoogleARCoreAugmentedFaceRegion Region);

	/**
	 * Returns the latest known local-to-tracking transform of the given face region.
	 */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|AugmentedImage")
	FTransform GetLocalToTrackingTransformOfRegion(EGoogleARCoreAugmentedFaceRegion Region);

private:
	void UpdateRegionTransforms(TMap<EGoogleARCoreAugmentedFaceRegion, FTransform>& InRegionLocalToTrackingTransforms);

	TMap<EGoogleARCoreAugmentedFaceRegion, FTransform> RegionLocalToTrackingTransforms;
};
