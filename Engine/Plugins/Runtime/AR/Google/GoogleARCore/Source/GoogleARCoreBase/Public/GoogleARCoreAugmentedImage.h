// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GoogleARCoreAugmentedImageDatabase.h"

#include "ARTrackable.h"

#include "GoogleARCoreAugmentedImage.generated.h"

/**
 * An object representing an augmented image currently in the scene.
 */
UCLASS(BlueprintType)
class GOOGLEARCOREBASE_API UGoogleARCoreAugmentedImage : public UARTrackedImage
{
	GENERATED_BODY()

public:

	/**
	 * Draw a box around the image, for debugging purposes.
	 *
	 * @param World				World context object.
	 * @param OutlineColor		Color of the lines.
	 * @param OutlineThickness	Thickness of the lines.
	 * @param PersistForSeconds	Number of seconds to keep the lines on-screen.
	 */
	virtual void DebugDraw(
		UWorld* World, const FLinearColor& OutlineColor,
		float OutlineThickness, float PersistForSeconds = 0.0f) const override;

	/**
	 * Update the tracked object.
	 *
	 * This is called by
	 * FGoogleARCoreAugmentedImageResource::UpdateGeometryData() to
	 * adjust the tracked position of the object.
	 *
	 * @param InTrackingSystem				The AR system to use.
	 * @param FrameNumber					The current frame number.
	 * @param Timestamp						The current time stamp.
	 * @param InLocalToTrackingTransform	Local to tracking space transformation.
	 * @param InAlignmentTransform			Alignment transform.
	 * @param InEstimatedSize				The estimated size of the object.
	 * @param InDetectedImage				The image that was found, in Unreal's cross-platform augmente reality API.
	 * @param InImageIndex					Which image in the augmented image database is being tracked.
	 * @param ImageName						The name of the image being tracked.
	 */
	void UpdateTrackedGeometry(
		const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem,
		uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform,
		const FTransform& InAlignmentTransform,
		FVector2D InEstimatedSize, UARCandidateImage* InDetectedImage,
		int32 InImageIndex, const FString& ImageName);

private:
	UPROPERTY()
	int32 ImageIndex;

	UPROPERTY()
	FString ImageName;

};

