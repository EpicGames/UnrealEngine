// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "Engine/Texture2D.h"

#include "GoogleARCoreAugmentedImageDatabase.generated.h"

#if PLATFORM_ANDROID
typedef struct ArAugmentedImageDatabase_ ArAugmentedImageDatabase;
#endif

/**
 * A single entry in a UGoogleARCoreAugmentedImageDatabase.
 *
 * Deprecated. Please use the cross-platform UARCandidateImage instead.
 */
USTRUCT(BlueprintType)
struct FGoogleARCoreAugmentedImageDatabaseEntry
{
	GENERATED_BODY()

	/**
	 * Name of the image. This can be retrieved from an active
	 * UGoogleARCoreAugmentedImage with the GetImageName function.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "GoogleARCore|AugmentedImages")
	FName Name;

	/**
	 * Texture to use for this image. Valid formats are RGBA8 and
	 * BGRA8.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "GoogleARCore|AugmentedImages")
	TObjectPtr<UTexture2D> ImageAsset;

	/**
	 * Width of the image in meters.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "GoogleARCore|AugmentedImages")
	float Width;

	FGoogleARCoreAugmentedImageDatabaseEntry()
		: ImageAsset(nullptr)
	    , Width(0.0f)
	{ }
};

/**
 * A collection of processed images for ARCore to track.
 *
 * Deprecated. Please use the ARCandidateImage list in UARSessionConfig instead.
 */

UCLASS(BlueprintType)
class GOOGLEARCOREBASE_API UGoogleARCoreAugmentedImageDatabase : public UDataAsset
{
	GENERATED_BODY()

	friend class FGoogleARCoreSession;

public:
	/**
	 * Overridden serialization function.
	 */
	virtual void Serialize(FArchive& Ar) override;

	/**
	 * The individual instances of
	 * FGoogleARCoreAugmentedImageDatabaseEntry objects.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "GoogleARCore|AugmentedImages")
	TArray<FGoogleARCoreAugmentedImageDatabaseEntry> Entries;

	/**
	 * The serialized database, in the ARCore augmented image database
	 * serialization format.
	 */
	UPROPERTY()
	TArray<uint8> SerializedDatabase;

private:
#if PLATFORM_ANDROID
	ArAugmentedImageDatabase* NativeHandle = nullptr;
#endif
};


