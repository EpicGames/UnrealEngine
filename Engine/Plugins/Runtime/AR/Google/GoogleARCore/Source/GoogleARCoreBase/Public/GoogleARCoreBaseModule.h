// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Stats/Stats.h"
#include "IHeadMountedDisplayModule.h"

/**
 * The public interface to this module.
 */
class IGoogleARCoreBaseModule : public IHeadMountedDisplayModule
{
public:

	/**
	 * Singleton-like access to this module's interface.  Provided for convenience only.
	 * Beware of calling this method during the shutdown phase because your module might
	 * already have been unloaded.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed.
	 */
	static inline IGoogleARCoreBaseModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IGoogleARCoreBaseModule >("GoogleARCoreBase");
	}

	/**
	 * Checks to see if this module is loaded and ready. It is only valid to call Get() if
	 * IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use.
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("GoogleARCoreBase");
	}
};

DECLARE_STATS_GROUP(TEXT("ARCore"), STATGROUP_ARCore, STATCAT_Advanced);
