// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GoogleARCoreCameraImage.generated.h"

typedef struct ArImage_ ArImage;
typedef struct ArSession_ ArSession;

/**
 * An object that represents an acquired CPU-accessible camera image.
 */
UCLASS(Blueprintable, BlueprintType)
class GOOGLEARCOREBASE_API UGoogleARCoreCameraImage : public UObject
{
	GENERATED_BODY()
public:

	~UGoogleARCoreCameraImage();

	/**
	 * Explicitly release the ARCore resources owned by this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCore|CameraImage", meta = (Keywords = "googlear arcore cameraimage"))
	void Release();

	/**
	 * Get the width of the image in pixels.
	 *
	 * @return The width.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCore|CameraImage", meta = (Keywords = "googlear arcore cameraimage"))
	int32 GetWidth() const;

	/**
	 * Get the height of the image in pixels.
	 *
	 * @return The height.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCore|CameraImage", meta = (Keywords = "googlear arcore cameraimage"))
	int32 GetHeight() const;

	/**
	 * Get the number of data planes in the image.
	 *
	 * @return The plane count.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCore|CameraImage", meta = (Keywords = "googlear arcore cameraimage"))
	int32 GetPlaneCount() const;

	/**
	 * Get the raw image data of a given plane.
	 */
	const uint8 *GetPlaneData(
		int32 Plane, int32 &PixelStride,
		int32 &RowStride, int32 &DataLength) const;

private:
#if PLATFORM_ANDROID
	ArImage *ArImage = nullptr;
	const ArSession* SessionHandle = nullptr;
#endif

	friend class FGoogleARCoreFrame;
};


