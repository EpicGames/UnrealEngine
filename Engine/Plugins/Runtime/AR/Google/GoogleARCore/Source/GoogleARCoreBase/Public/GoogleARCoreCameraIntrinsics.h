// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"

#include "GoogleARCoreTypes.h"
#include "ARTypes.h"
#include "GoogleARCoreCameraIntrinsics.generated.h"

class FGoogleARCoreSession;

#if PLATFORM_ANDROID
typedef struct ArCameraIntrinsics_ ArCameraIntrinsics;
#endif

/**
 * An object wrapping the ArCameraIntrinsics data from the ARCore SDK.
 */
UCLASS(BlueprintType)
class GOOGLEARCOREBASE_API UGoogleARCoreCameraIntrinsics : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Get the focal length in pixels.
	 *
	 * @param OutFX  The focal length on the X axis.
	 * @param OutFY  The focal length on the Y axis.
	 */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|CameraIntrinsics")
	void GetFocalLength(float &OutFX, float &OutFY);

	/**
	 * Get the principal point in pixels.
	 *
	 * @param OutCX  The principle point on the X axis.
	 * @param OutCY  The principle point the Y axis.
	 */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|CameraIntrinsics")
	void GetPrincipalPoint(float &OutCX, float &OutCY);

	/**
	 * Get the image's width and height in pixels.
	 *
	 * @param OutWidth   The width.
	 * @param OutHeight  The height.
	 */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|CameraIntrinsics")
	void GetImageDimensions(int32 &OutWidth, int32 &OutHeight);
	
	void SetCameraIntrinsics(const FARCameraIntrinsics& InCameraIntrinsics);

private:
	FARCameraIntrinsics CameraIntrinsics;
};

