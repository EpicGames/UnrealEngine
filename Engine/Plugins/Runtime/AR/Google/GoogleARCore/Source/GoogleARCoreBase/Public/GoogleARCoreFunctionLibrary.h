// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "GoogleARCoreTypes.h"
#include "GoogleARCoreFunctionLibrary.generated.h"

/** A function library that provides static/Blueprint functions associated with GoogleARCore session.*/
UCLASS()
class GOOGLEARCOREBASE_API UGoogleARCoreSessionFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Create an ARCandidateImage object from the raw pixel data and add it to the ARCandidateImageList of the given \c UARSessionConfig object.
	 *
	 * Note that you need to restart the AR session with the \c UARSessionConfig you are adding to to make the change take effect.
	 *
	 * On ARCore platform, you can leave the PhysicalWidth and PhysicalHeight to 0 if you don't know the physical size of the image or
	 * the physical size is dynamic. And this function takes time to perform non-trivial image processing (20ms - 30ms),
	 * and should be run on a background thread.
	 *
	 * @return A \c UARCandidateImage Object pointer if the underlying ARPlatform added the candidate image at runtime successfully.
	 *		  Return nullptr otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Session", meta = (Keywords = "ar augmentedreality augmented reality candidate image"))
	static UARCandidateImage* AddRuntimeCandidateImageFromRawbytes(UARSessionConfig* SessionConfig, const TArray<uint8>& ImageGrayscalePixels, int ImageWidth, int ImageHeight,
			FString FriendlyName, float PhysicalWidth, UTexture2D* CandidateTexture = nullptr);
};

/** A function library that provides static/Blueprint functions associated with most recent GoogleARCore tracking frame.*/
UCLASS()
class GOOGLEARCOREBASE_API UGoogleARCoreFrameFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

#if PLATFORM_ANDROID
	/**
	 * Gets the camera metadata for the latest camera image.
	 * Note that ACameraMetadata is a Ndk type. Include the Ndk header <camera/NdkCameraMetadata.h> to use query value from ACameraMetadata.
	 *
	 * @param OutCameraMetadata		A pointer to a ACameraMetadata struct which is only valid in one frame.
	 * @return An EGoogleARCoreFunctionStatus. Possible value: Success, SessionPaused, NotAvailable.
	 */
	UE_DEPRECATED(5.3, "GetCameraMetadata is deprecated and non-functional.  It does not have an exact replacement.  ArImageMetadata seems to be the closest thing, but we do not currently expose it.")
	static EGoogleARCoreFunctionStatus GetCameraMetadata(const ACameraMetadata*& OutCameraMetadata) { return EGoogleARCoreFunctionStatus::Unknown; }
#endif
};
