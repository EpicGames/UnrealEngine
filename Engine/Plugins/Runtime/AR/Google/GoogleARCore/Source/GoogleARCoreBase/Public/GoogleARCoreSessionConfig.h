// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "GoogleARCoreTypes.h"
#include "ARSessionConfig.h"
#include "GoogleARCoreAugmentedImageDatabase.h"

#include "GoogleARCoreSessionConfig.generated.h"

/**
 * A UDataAsset that can be used to configure ARCore specific settings on top of
 * UARSessionConfig. 
 */
UCLASS(BlueprintType, Category = "AR AugmentedReality")
class GOOGLEARCOREBASE_API UGoogleARCoreSessionConfig : public UARSessionConfig
{
	GENERATED_BODY()

	// We keep the type here so that we could extend ARCore specific session config later.

public:

	/**
	 * Get the augmented image database being used.
	 */
	UFUNCTION(BlueprintPure, Category = "google arcore augmentedimages")
	UGoogleARCoreAugmentedImageDatabase* GetAugmentedImageDatabase()
	{
		return AugmentedImageDatabase;
	}

	/**
	 * Set the augmented image database to use.
	 */
	UFUNCTION(BlueprintCallable, Category = "google arcore augmentedimages")
	void SetAugmentedImageDatabase(UGoogleARCoreAugmentedImageDatabase* NewImageDatabase)
	{
		AugmentedImageDatabase = NewImageDatabase;
	}

	/**
	 * A UGoogleARCoreAugmentedImageDatabase asset to use use for
	 * image tracking.
	 */
	UPROPERTY(EditAnywhere, Category = "google arcore augmentedimages")
	TObjectPtr<UGoogleARCoreAugmentedImageDatabase> AugmentedImageDatabase;

	/** Configure which camera will be used to in the AR session. */
	UPROPERTY(EditAnywhere, Category = "ARCore Settings")
	EGoogleARCoreCameraFacing CameraFacing = EGoogleARCoreCameraFacing::Back;

	/** Configure which Augmented Face mode will be used in the AR session. */
	UPROPERTY(EditAnywhere, Category = "ARCore Settings")
	EGoogleARCoreAugmentedFaceMode AugmentedFaceMode = EGoogleARCoreAugmentedFaceMode::Disabled;
public:

	/**
	 * Create a new ARCore session configuration.
	 *
	 * @param bHorizontalPlaneDetection			True to enable horizontal plane detection.
	 * @param bVerticalPlaneDetection			True to enable vertical plane detection.
	 * @param InLightEstimationMode				The light estimation mode to use.
	 * @param InFrameSyncMode					The frame synchronization mode to use.
	 * @param bInEnableAutoFocus				True to enable auto-focus.
	 * @param bInEnableAutomaticCameraOverlay	True to enable the camera overlay.
	 * @param bInEnableAutomaticCameraTracking	True to enable automatic camera tracking.
	 * @return The new configuration object.
	 */
	static UGoogleARCoreSessionConfig* CreateARCoreSessionConfig(bool bHorizontalPlaneDetection, bool bVerticalPlaneDetection, EARLightEstimationMode InLightEstimationMode, EARFrameSyncMode InFrameSyncMode, bool bInEnableAutoFocus, bool bInEnableAutomaticCameraOverlay, bool bInEnableAutomaticCameraTracking)
	{
		UGoogleARCoreSessionConfig* NewARCoreConfig = NewObject<UGoogleARCoreSessionConfig>();
		NewARCoreConfig->bHorizontalPlaneDetection = bHorizontalPlaneDetection;
		NewARCoreConfig->bVerticalPlaneDetection = bVerticalPlaneDetection;
		NewARCoreConfig->LightEstimationMode = InLightEstimationMode;
		NewARCoreConfig->bEnableAutoFocus = bInEnableAutoFocus;
		NewARCoreConfig->FrameSyncMode = InFrameSyncMode;
		NewARCoreConfig->bEnableAutomaticCameraOverlay = bInEnableAutomaticCameraOverlay;
		NewARCoreConfig->bEnableAutomaticCameraTracking = bInEnableAutomaticCameraTracking;
		NewARCoreConfig->AugmentedImageDatabase = nullptr;

		return NewARCoreConfig;
	};
};
