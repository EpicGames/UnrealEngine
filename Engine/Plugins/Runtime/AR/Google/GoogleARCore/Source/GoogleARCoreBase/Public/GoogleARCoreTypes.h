// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "ARTypes.h"
#include "ARTrackable.h"
#include "ARTraceResult.h"
#include "ARSystem.h"
#include "ARPin.h"

#include "GoogleARCoreTypes.generated.h"

/// @defgroup GoogleARCoreBase Google ARCore Base
/// The base module for Google ARCore plugin

#if PLATFORM_ANDROID
// Include <Camera/NdkCameraMetadata.h> to use this type.
typedef struct ACameraMetadata ACameraMetadata;

// Forward decalare type defined in arcore_c_api.h
typedef struct ArTrackable_ ArTrackable;
typedef struct ArPlane_ ArPlane;
typedef struct ArPoint_ ArPoint;
typedef struct ArPointCloud_ ArPointCloud;
typedef struct ArAnchor_ ArAnchor;
#endif

enum class EGoogleARCoreAvailability : uint8
{
	/* An internal error occurred while determining ARCore availability. */
	UnknownError = 0,
	/* ARCore is not installed, and a query has been issued to check if ARCore is is supported. */
	UnknownChecking = 1,
	/*
	 * ARCore is not installed, and the query to check if ARCore is supported
	 * timed out. This may be due to the device being offline.
	 */
	UnknownTimedOut = 2,
	/* ARCore is not supported on this device.*/
	UnsupportedDeviceNotCapable = 100,
	/* The device and Android version are supported, but the ARCore APK is not installed.*/
	SupportedNotInstalled = 201,
	/*
	 * The device and Android version are supported, and a version of the
	 * ARCore APK is installed, but that ARCore APK version is too old.
	 */
	SupportedApkTooOld = 202,
	/* ARCore is supported, installed, and available to use. */
	SupportedInstalled = 203
};

enum class EGoogleARCoreInstallStatus : uint8
{
	/* The requested resource is already installed.*/
	Installed = 0,
	/* Installation of the resource was requested. The current activity will be paused. */
	Requrested = 1,
};

/**
 * @ingroup GoogleARCoreBase
 * Describes the status of most ARCore functions.
 */
UENUM(BlueprintType)
enum class EGoogleARCoreFunctionStatus : uint8
{
	/** Function returned successfully. */
	Success,
	/** Function failed due to Fatal error. */
	Fatal,
	/** Function failed due to the session isn't running. */
	SessionPaused,
	/** Function failed due to ARCore session isn't in tracking state. */
	NotTracking,
	/** Function failed due to the requested resource is exhausted. */
	ResourceExhausted,
	/** Function failed due to ARCore session hasn't started or the requested resource isn't available yet. */
	NotAvailable,
	/** Function failed due to the function augment has invalid type. */
	InvalidType,
	/** Function failed due to it is invoked at an illegal or inappropriate time. */
	IllegalState,
	/** Function failed with unknown reason. */
	Unknown
};

/**
 * @ingroup GoogleARCoreBase
 * Describes the tracking state of the current ARCore session.
 */
enum class EGoogleARCoreTrackingState : uint8
{
	/** Tracking is valid. */
	Tracking = 0,
	/** Tracking is temporary lost but could recover in the future. */
	NotTracking = 1,
	/** Tracking is lost will not recover. */
	StoppedTracking = 2
};

/**
* @ingroup GoogleARCoreBase
* 2d coordinate systems referenced by ARCore.
*/
enum class EGoogleARCoreCoordinates2DType : uint8
{
	/** ARCore normalized pass-through camera texture(for GPU usage) space with top-left (0.0f, 0.0f) and bottom-right(1.0f, 1.0f). */
	Texture = 0,
	/** ARCore normalized pass-through camera image(for CPU usage) space with top-left (0.0f, 0.0f) and bottom-right(1.0f, 1.0f). */
	Image = 1,
	/** UE normalized viewport space with top-left(0,0f ,0.0f) and bottom-right(1.0f, 1.0f) */
	Viewport = 2
};


/**
 * A struct describes the ARCore light estimation.
 */
struct FGoogleARCoreLightEstimate
{
	/** Whether this light estimation is valid. */
	bool bIsValid = false;

	/** The average pixel intensity of the passthrough camera image. */
	float PixelIntensity = 0.0f;

	/**
	 * The RGB scale to match the color of the light in the real environment.
	 */
	FVector RGBScaleFactor = FVector(0.0f);
};

/**
 * @ingroup GoogleARCoreBase
 * Describes which channel ARLineTrace will be performed on.
 */
UENUM(BlueprintType, Category = "GoogleARCore|TraceChannel", meta = (Bitflags))
enum class EGoogleARCoreLineTraceChannel : uint8
{
	None = 0,
	/** Trace against feature point cloud. */
	FeaturePoint = 1,
	/** Trace against the infinite plane. */
	InfinitePlane = 2,
	/** Trace against the plane using its extent. */
	PlaneUsingExtent = 4,
	/** Trace against the plane using its boundary polygon. */
	PlaneUsingBoundaryPolygon = 8,
	/**
	 * Trace against feature point and attempt to estimate the normal of the surface centered around the trace hit point.
	 * Surface normal estimation is most likely to succeed on textured surfaces and with camera motion.
	 */
	FeaturePointWithSurfaceNormal = 16,
	/** Trace against augmented images. */
	AugmentedImage = 32,
};
ENUM_CLASS_FLAGS(EGoogleARCoreLineTraceChannel);

/**
 * @ingroup GoogleARCoreBase
 * Describes the orientation of the selected camera relative to the device display.
 */
UENUM(BlueprintType, Category = "GoogleARCore|CameraConfig")
enum class EGoogleARCoreCameraFacing : uint8
{
	None = 0 UMETA(Hidden),
	/* Camera facing away from user. */
	Back = 1,
	/* Camera facing towards user. */
	Front = 2
};

/**
 * @ingroup GoogleARCoreBase
 * Describes the possible modes for Augmented Face detection.
 */
UENUM(BlueprintType, Category = "GoogleARCore|AugmentedFace")
enum class EGoogleARCoreAugmentedFaceMode : uint8
{
	/* A mode where AugmentedFace detection is disabled. */
	Disabled = 0,
	/* A mode where AugmentedFace detection performs face pose, region pose and face mesh estimation. */
	PoseAndMesh = 2,
};

/**
 * @ingroup GoogleARCoreBase
 * Describes the possible tracking failure reasons in ARCore.
 */
UENUM(BlueprintType, Category = "GoogleARCore|AugmentedFace")
enum class EGoogleARCoreTrackingFailureReason : uint8
{
	/** Tracking is working normally, or ARCore session is not currently running. */
	None = 0,
	/**
	 * Tracking lost due to bad internal state. Please try restarting the AR experience.
	 * This should be seen rarely, and should be reported to and fixed by ARCore team.
	 */
	BadState = 1,
	/** Tracking lost due to poor lighting conditions. Please move to a more brightly lit area */
	InsufficientLight = 2,
	/** Tracking lost due to excessive motion. Please  move device more slowly. */
	ExcessiveMotion = 3,
	/**
	 * Tracking lost due to insufficient trackable features. Please move to area with more features,
	 * such as corners, objects, surfaces with texture. Avoid blank walls and surfaces without detail.
	 */
	InsufficientFeatures = 4
};

/**
 * @ingroup GoogleARCoreBase
 * Describes the target FPS of the ARCore camera.
 */
UENUM(BlueprintType, Category = "GoogleARCore|CameraConfig")
enum class EGoogleARCoreCameraFPS : uint8
{
	/** Targeting any FPS. */
	FPS_Any = 0,

	/** Targeting 30 FPS. */
	FPS_30 = 1 << 0,

	/** Targeting 60 FPS. */
	FPS_60 = 1 << 1,
};

/**
 * @ingroup GoogleARCoreBase
 * Describes the usage of the depth sensor of the ARCore camera.
 */
UENUM(BlueprintType, Category = "GoogleARCore|CameraConfig")
enum class EGoogleARCoreCameraDepthSensorUsage : uint8
{
	/** Do not specify the depth sensor usage. */
	DepthSensor_Any = 0,

	/** Require and use the depth sensor. */
	DepthSensor_RequireAndUse = 1 << 0,

	/** Do not use the depth sensor. */
	DepthSensor_DoNotUse = 1 << 1,
};

/**
 * Camera configuration from ARCore.
 */
USTRUCT(BlueprintType)
struct GOOGLEARCOREBASE_API FGoogleARCoreCameraConfig
{
	GENERATED_BODY()

	/**
	 * CPU-accessible camera image resolution.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "GoogleARCore|CameraConfig")
	FIntPoint CameraImageResolution = FIntPoint::ZeroValue;

	/**
	 * Texture resolution for the camera image accessible to the
	 * graphics API and shaders.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "GoogleARCore|CameraConfig")
	FIntPoint CameraTextureResolution = FIntPoint::ZeroValue;

	/** The id of the camera will be used in this CameraConfig. */
	UPROPERTY(BlueprintReadOnly, Category = "GoogleARCore|CameraConfig")
	FString CameraID;

	/** The target FPS of the camera. */
	UPROPERTY(BlueprintReadOnly, Category = "GoogleARCore|CameraConfig", meta = (Bitmask, BitmaskEnum = "/Script/GoogleARCoreBase.EGoogleARCoreCameraFPS"))
	int32 TargetFPS = 0; // EGoogleARCoreCameraFPS::FPS_Any

	/** The desired depth sensor usage of the camera. */
	UPROPERTY(BlueprintReadOnly, Category = "GoogleARCore|CameraConfig", meta = (Bitmask, BitmaskEnum = "/Script/GoogleARCoreBase.EGoogleARCoreCameraDepthSensorUsage"))
	int32 DepthSensorUsage = 0; // EGoogleARCoreCameraDepthSensorUsage::DepthSensor_Any

	EGoogleARCoreCameraFacing CameraFacing = EGoogleARCoreCameraFacing::Back;

	/** Below are for internal use only */
	FGoogleARCoreCameraConfig();
	bool IsCompatibleWith(const FGoogleARCoreCameraConfig& OtherConfig) const;
	int32 GetMaxFPS() const;
	void SetMaxFPS(int32 MaxFPS);
	FString ToLogString() const;
};

/**
 * Delegate to call on camera configuration.
 */
class GOOGLEARCOREBASE_API FGoogleARCoreDelegates
{
public:
/// @cond EXCLUDE_FROM_DOXYGEN
	DECLARE_MULTICAST_DELEGATE_OneParam(FGoogleARCoreOnConfigCameraDelegate, const TArray<FGoogleARCoreCameraConfig>&);
/// @endcond

	/**
	 * A delegate can be bound through C++. It will be called before
	 * ARSession started and returns a list of supported ARCore camera
	 * configurations. Bind this delegate if you want to choose a
	 * specific camera config in your app. Call
	 * UGoogleARCoreSessionFunctionLibrary::ConfigARCoreCamera after
	 * the delegate is triggered.
	 */
	static FGoogleARCoreOnConfigCameraDelegate OnCameraConfig;
};

/**
 * Manager for ARCore delegates.
 */
UCLASS(BlueprintType, Category = "AR AugmentedReality")
class GOOGLEARCOREBASE_API UGoogleARCoreEventManager : public UObject
{
	GENERATED_BODY()
public:
/// @cond EXCLUDE_FROM_DOXYGEN
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FGoogleARCoreOnConfigCameraDynamicDelegate, const TArray<FGoogleARCoreCameraConfig>&, SupportedCameraConfig);
/// @endcond

	UGoogleARCoreEventManager()
	{
		RegisterDelegates();
	}

	~UGoogleARCoreEventManager()
	{
		UnregisterDelegates();
	}

	/**
	 * A dynamic delegate can be assigned through blueprint. Will be called before ARSession started and returns
	 * an array of supported ARCore camera config.
	 *
	 * The array will always return 3 camera configs. The GPU texture resolutions
	 * are the same in all three configs. Currently, most devices provide GPU
	 * texture resolution of 1920 x 1080, but devices might provide higher or lower
	 * resolution textures, depending on device capabilities. The CPU image
	 * resolutions returned are VGA, 720p, and a resolution matching the GPU
	 * texture.
	 *
	 * Bind this delegate if you want to choose a specific camera config in your app. Call
	 * UGoogleARCoreSessionFunctionLibrary::ConfigARCoreCamera after the delegate is triggered.
	 */
	UPROPERTY(BlueprintAssignable)
	FGoogleARCoreOnConfigCameraDynamicDelegate OnConfigCamera;

private:
	void RegisterDelegates()
	{
		FGoogleARCoreDelegates::OnCameraConfig.AddUObject(this, &UGoogleARCoreEventManager::OnConfigCamera_Handler);
	}
	void UnregisterDelegates()
	{
		FGoogleARCoreDelegates::OnCameraConfig.RemoveAll(this);
	}

	void OnConfigCamera_Handler(const TArray<FGoogleARCoreCameraConfig>& SupportedCameraConfig)
	{
		OnConfigCamera.Broadcast(SupportedCameraConfig);
	}
};

class FGoogleARCoreSession;
/**
 * A UObject that contains a set of observed 3D points and confidence values.
 */
UCLASS(BlueprintType)
class GOOGLEARCOREBASE_API UGoogleARCorePointCloud : public UObject
{
	friend class FGoogleARCoreFrame;
	friend class FGoogleARCoreSession;

	GENERATED_BODY()
public:
	/** Destructor */
	~UGoogleARCorePointCloud();

	/** Returns the timestamp in nanosecond when this point cloud was observed. */
	int64 GetUpdateTimestamp();

	/** Checks if this point cloud has been updated in this frame. */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|PointCloud")
	bool IsUpdated();

	/** Returns the number of point inside this point cloud. */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|PointCloud")
	int GetPointNum();

	/** Returns the point position in Unreal world space and it's confidence value from 0 ~ 1. */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|PointCloud")
	void GetPoint(int Index, FVector& OutWorldPosition, float& OutConfidence);

	/**
	 * Returns the point Id of the point at the given index.
	 *
	 * Each point has a unique identifier (within a session) that is persistent
	 * across frames. That is, if a point from point cloud 1 has the same id as the
	 * point from point cloud 2, then it represents the same point in space.
	 */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|PointCloud")
	int GetPointId(int Index);

	/** Returns the point position in Unreal AR Tracking space. */
	UFUNCTION(BlueprintPure, Category = "GoogleARCore|PointCloud")
	void GetPointInTrackingSpace(int Index, FVector& OutTrackingSpaceLocation, float& OutConfidence);

	/** Release PointCloud's resources back to ArCore. Data will not be available after releasePointCloud is called. */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCore|PointCloud")
	void ReleasePointCloud();

private:
	TWeakPtr<FGoogleARCoreSession> Session;
	bool bIsUpdated = false;
#if PLATFORM_ANDROID
	ArPointCloud* PointCloudHandle;
#endif
};

/**
* Helper class used to expose FGoogleARCoreSessionConfig setting in the Editor plugin settings.
*/
UCLASS(config = Engine, defaultconfig)
class GOOGLEARCOREBASE_API UGoogleARCoreEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Check this option if you app require ARCore to run on Android. */
	UPROPERTY(EditAnywhere, config, Category = "ARCore Plugin Settings", meta = (ShowOnlyInnerProperties))
	bool bARCoreRequiredApp = true;
};
