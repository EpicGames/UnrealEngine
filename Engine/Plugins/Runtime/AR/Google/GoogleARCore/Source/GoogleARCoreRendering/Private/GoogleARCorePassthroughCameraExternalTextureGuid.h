// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

/* GUID used to indicate that this external texture is the tango camera. */
static FGuid GoogleARCorePassthroughCameraExternalTextureGuid(0x3c87151d, 0x929e4f87, 0xab5bb2dc, 0xc4f9042c);

