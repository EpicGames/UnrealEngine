// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCorePassthroughCameraRenderer.h"
#include "ScreenRendering.h"
#include "RendererInterface.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EngineModule.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "SceneUtils.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialDomain.h"
#include "MaterialShader.h"
#include "MaterialShaderType.h"
#include "Materials/MaterialRenderProxy.h"
#include "CommonRenderResources.h"
#include "ARUtilitiesFunctionLibrary.h"
#include "HAL/IConsoleManager.h"
#include "UObject/Package.h"
#include "PostProcess/DrawRectangle.h"
#include "RHIResourceUtils.h"
#include "RHIStaticStates.h"
#include "MediaShaders.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define ALLOWS_DEBUG_OVERLAY 1
#else
	#define ALLOWS_DEBUG_OVERLAY 0
#endif

static int32 GDebugOverlayMode = 0;
static FAutoConsoleVariableRef CVarDebugOverlayMode(
	TEXT("arcore.DebugOverlayMode"),
	GDebugOverlayMode,
	TEXT("The debug overlay mode for ARCore:\n")
	TEXT("0: Disabled (Default)\n")
	TEXT("1: Show the scene depth map texture\n")
	TEXT("2: Show coloration of the scene depth data\n")
	);

enum class EARCoreDebugOverlayMode : uint8
{
	None = 0,
	SceneDepthMap,
	SceneDepthColoration,
};

FGoogleARCorePassthroughCameraRenderer::FGoogleARCorePassthroughCameraRenderer()
{
	auto MaterialLoader = GetDefault<UGoogleARCoreCameraOverlayMaterialLoader>();
	RegularOverlayMaterial = UMaterialInstanceDynamic::Create(MaterialLoader->RegularOverlayMaterial, GetTransientPackage());
	DebugOverlayMaterial = UMaterialInstanceDynamic::Create(MaterialLoader->DebugOverlayMaterial, GetTransientPackage());
	DepthColorationMaterial = UMaterialInstanceDynamic::Create(MaterialLoader->DepthColorationMaterial, GetTransientPackage());
	DepthOcclusionMaterial = UMaterialInstanceDynamic::Create(MaterialLoader->DepthOcclusionMaterial, GetTransientPackage());
}

void FGoogleARCorePassthroughCameraRenderer::InitializeRenderer_RenderThread(FSceneViewFamily& InViewFamily)
{
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	if (!OverlayIndexBufferRHI)
	{
		// Initialize Index buffer;
		const uint16 Indices[] = { 0, 1, 2, 2, 1, 3};

		// Create index buffer. Fill buffer with initial data upon creation
		OverlayIndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("OverlayIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
	}
	
	if (!OverlayVertexBufferRHI)
	{
		// Unreal uses reversed z. 0 is the farthest.
		const FFilterVertex Vertices[] =
		{
			FFilterVertex{ FVector4f(0, 0, 0, 1), FVector2f(0, 0) },
			FFilterVertex{ FVector4f(0, 1, 0, 1), FVector2f(0, 1) },
			FFilterVertex{ FVector4f(1, 0, 0, 1), FVector2f(1, 0) },
			FFilterVertex{ FVector4f(1, 1, 0, 1), FVector2f(1, 1) },
		};

		OverlayVertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("OverlayVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Vertices));
	}
}

class FPostProcessMaterialShader : public FMaterialShader
{
public:
	FPostProcessMaterialShader() = default;
	FPostProcessMaterialShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMaterialShader(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.MaterialParameters.MaterialDomain == MD_PostProcess
			&& IsMobilePlatform(Parameters.Platform)
			// Note: IsMobilePlatform can be set for non-android platforms
			&& IsAndroidPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_MATERIAL"), 1);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_MATERIAL_BEFORE_TONEMAP"), (Parameters.MaterialParameters.BlendableLocation != BL_SceneColorAfterTonemapping) ? 1 : 0);

	}
};

// We use something similar to the PostProcessMaterial to render the color camera overlay.
class FGoogleARCoreCameraOverlayVS : public FPostProcessMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FGoogleARCoreCameraOverlayVS, Material);

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPostProcessMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_AR_PASSTHROUGH"), 1);
	}

	FGoogleARCoreCameraOverlayVS() = default;
	FGoogleARCoreCameraOverlayVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FPostProcessMaterialShader(Initializer)
	{}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View)
	{
		UE::Renderer::PostProcess::SetDrawRectangleParameters(BatchedParameters, this, View);
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGoogleARCoreCameraOverlayVS, "/Engine/Private/PostProcessMaterialShaders.usf", "MainVS", SF_Vertex);

class FGoogleARCoreCameraOverlayPS : public FPostProcessMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FGoogleARCoreCameraOverlayPS, Material);

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPostProcessMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("OUTPUT_GAMMA_SPACE"), IsMobileHDR() ? 0 : 1);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_AR_PASSTHROUGH"), 1);
	}

	FGoogleARCoreCameraOverlayPS() = default;
	FGoogleARCoreCameraOverlayPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FPostProcessMaterialShader(Initializer)
	{}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FMaterialRenderProxy* MaterialProxy, const FMaterial& Material)
	{
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
		FMaterialShader::SetParameters(BatchedParameters, MaterialProxy, Material, View);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGoogleARCoreCameraOverlayPS, "/Engine/Private/PostProcessMaterialShaders.usf", "MainPS", SF_Pixel);

// Postprocess pixel shader for ARCore camera performing YCbCr conversion.
class FGoogleARCoreCameraOverlayYCbCrConversionPS : public FPostProcessMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FGoogleARCoreCameraOverlayYCbCrConversionPS, Material);

	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FGoogleARCoreCameraOverlayYCbCrConversionPS, FPostProcessMaterialShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FYCbCrConversionParameters, )
		SHADER_PARAMETER(FMatrix44f, YCbCrColorTransform)
		SHADER_PARAMETER(uint32, YCbCrSrgbToLinear)
	END_SHADER_PARAMETER_STRUCT()
	using FParameters = FYCbCrConversionParameters;

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPostProcessMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("OUTPUT_GAMMA_SPACE"), IsMobileHDR() ? 0 : 1);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_AR_PASSTHROUGH"), 1);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_AR_YCBCR_CONVERSION"), 1);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FMaterialRenderProxy* MaterialProxy, const FMaterial& Material)
	{
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
		FMaterialShader::SetParameters(BatchedParameters, MaterialProxy, Material, View);
	}

	FYCbCrConversionParameters GetYCbCrConversionParameters(const FYCbCrConversion& YCbCrConversion) const
	{
		FMatrix YCbCrModelConversionMatrix = FMatrix::Identity;
		FVector YCbCrRangeOffset = FVector::ZeroVector;

		check(YCbCrConversion.YCbCrRange != EYCbCrRange::Unknown);
		const bool bFullRange = (YCbCrConversion.YCbCrRange == EYCbCrRange::Full);

		switch (YCbCrConversion.YCbCrModelConversion)
		{
		case EYCbCrModelConversion::YCbCrIdentity:
			// Use identity matrix
			break;

		case EYCbCrModelConversion::YCbCrRec709:
			YCbCrModelConversionMatrix = (bFullRange) ? MediaShaders::YuvToRgbRec709Unscaled : MediaShaders::YuvToRgbRec709Scaled;
			break;

		case EYCbCrModelConversion::YCbCrRec601:
			YCbCrModelConversionMatrix = (bFullRange) ? MediaShaders::YuvToRgbRec601Unscaled : MediaShaders::YuvToRgbRec601Scaled;
			break;

		case EYCbCrModelConversion::YCbCrRec2020:
			YCbCrModelConversionMatrix = (bFullRange) ? MediaShaders::YuvToRgbRec2020Unscaled : MediaShaders::YuvToRgbRec2020Scaled;
			break;

		case EYCbCrModelConversion::None: // None value is not expected, otherwise it should be using FGoogleARCoreCameraOverlayPS shader instead.
		default:
			UE_LOGF(LogTemp, Error, "Unexpected YCbCr Model conversion: %d", EnumToUnderlyingType(YCbCrConversion.YCbCrModelConversion));
			break;
		}

		switch (YCbCrConversion.NumBits)
		{
		case 8:  YCbCrRangeOffset = bFullRange ? MediaShaders::YUVOffsetNoScale8bits : MediaShaders::YUVOffset8bits; break;
		case 10: YCbCrRangeOffset = bFullRange ? MediaShaders::YUVOffsetNoScale10bits : MediaShaders::YUVOffset10bits; break;
		case 16: YCbCrRangeOffset = bFullRange ? MediaShaders::YUVOffsetNoScale16bits : MediaShaders::YUVOffset16bits; break;
		case 32: YCbCrRangeOffset = bFullRange ? MediaShaders::YUVOffsetNoScaleFloat : MediaShaders::YUVOffsetFloat; break;
		default:
			UE_LOGF(LogTemp, Error, "Unexpected number of bits in YCbCr conversion: %d", YCbCrConversion.NumBits);
			break;
		}

		FYCbCrConversionParameters YCbCrConversionParameters;
		YCbCrConversionParameters.YCbCrColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(YCbCrModelConversionMatrix, YCbCrRangeOffset);
		YCbCrConversionParameters.YCbCrSrgbToLinear = 1;
		return YCbCrConversionParameters;
	}
};

IMPLEMENT_GLOBAL_SHADER(FGoogleARCoreCameraOverlayYCbCrConversionPS, "/Engine/Private/PostProcessMaterialShaders.usf", "MainPS", SF_Pixel);

void FGoogleARCorePassthroughCameraRenderer::RenderVideoOverlayWithMaterial(FRHICommandList& RHICmdList, FSceneView& InView, UMaterialInstanceDynamic* OverlayMaterialToUse, bool bRenderingOcclusion, bool bUsesCameraTexture)
{
#if PLATFORM_ANDROID
	if (FAndroidMisc::ShouldUseVulkan() && IsMobileHDR() && !RHICmdList.IsInsideRenderPass())
	{
		// We must NOT call DrawIndexedPrimitive below if not in a render pass on Vulkan, it's very likely to crash!
		UE_LOGF(LogTemp, Warning, "FGoogleARCorePassthroughCameraRenderer::RenderVideoOverlayWithMaterial: skipped due to not called within a render pass on Vulkan!");
		return;
	}
	
	if (!OverlayMaterialToUse || !OverlayMaterialToUse->IsValidLowLevel())
	{
		return;
	}
	
	SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, RenderVideoOverlay_Occlusion ,  bRenderingOcclusion, TEXT("VideoOverlay (Occlusion)" ));
	SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, RenderVideoOverlay_Background, !bRenderingOcclusion, TEXT("VideoOverlay (Background)"));

	const auto FeatureLevel = InView.GetFeatureLevel();
	IRendererModule& RendererModule = GetRendererModule();

	if (FeatureLevel <= ERHIFeatureLevel::ES3_1)
	{
		const FMaterialRenderProxy* MaterialProxy = OverlayMaterialToUse->GetRenderProxy();
		const FMaterial& CameraMaterial = MaterialProxy->GetMaterialWithFallback(FeatureLevel, MaterialProxy);
		const FMaterialShaderMap* MaterialShaderMap = CameraMaterial.GetRenderingThreadShaderMap();

		TShaderRef<FGoogleARCoreCameraOverlayPS> PixelShader = MaterialShaderMap->GetShader<FGoogleARCoreCameraOverlayPS>();
		TShaderRef<FGoogleARCoreCameraOverlayVS> VertexShader = MaterialShaderMap->GetShader<FGoogleARCoreCameraOverlayVS>();

		const bool bCameraYCbCrConversionEnabled = bUsesCameraTexture && CameraYCbCrConversion.YCbCrModelConversion != EYCbCrModelConversion::None;
		TShaderRef<FGoogleARCoreCameraOverlayYCbCrConversionPS> PixelShaderWithYCbCrConversion = MaterialShaderMap->GetShader<FGoogleARCoreCameraOverlayYCbCrConversionPS>();

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		
		if (bRenderingOcclusion)
		{
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		}
		else
		{
			// Disable the write mask for the alpha channel so that the scene depth info saved in it is retained
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
		}
		
		//GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		//GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		if (bCameraYCbCrConversionEnabled)
		{
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShaderWithYCbCrConversion.GetPixelShader();
		}
		else
		{
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		}
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParametersLegacyVS(RHICmdList, VertexShader, InView);
		if (bCameraYCbCrConversionEnabled)
		{
			const FGoogleARCoreCameraOverlayYCbCrConversionPS::FYCbCrConversionParameters YCbCrConversionParameters = PixelShaderWithYCbCrConversion->GetYCbCrConversionParameters(CameraYCbCrConversion);
			SetShaderParametersMixedPS(RHICmdList, PixelShaderWithYCbCrConversion, YCbCrConversionParameters, InView, MaterialProxy, CameraMaterial);
		}
		else
		{
			SetShaderParametersLegacyPS(RHICmdList, PixelShader, InView, MaterialProxy, CameraMaterial);
		}

		if (OverlayVertexBufferRHI.IsValid() && OverlayIndexBufferRHI.IsValid())
		{
			RHICmdList.SetStreamSource(0, OverlayVertexBufferRHI, 0);
			RHICmdList.DrawIndexedPrimitive(
				OverlayIndexBufferRHI,
				/*BaseVertexIndex=*/ 0,
				/*MinIndex=*/ 0,
				/*NumVertices=*/ 4,
				/*StartIndex=*/ 0,
				/*NumPrimitives=*/ 2,
				/*NumInstances=*/ 1
			);
		}
	}
#endif
}

void FGoogleARCorePassthroughCameraRenderer::RenderVideoOverlay_RenderThread(FRHICommandList& RHICmdList, FSceneView& InView)
{
	auto OverlayMaterialToUse = RegularOverlayMaterial;
	
#if ALLOWS_DEBUG_OVERLAY
	if (GDebugOverlayMode == (int32)EARCoreDebugOverlayMode::SceneDepthColoration)
	{
		OverlayMaterialToUse = DepthColorationMaterial;
	}
	else if (GDebugOverlayMode == (int32)EARCoreDebugOverlayMode::SceneDepthMap)
	{
		OverlayMaterialToUse = DebugOverlayMaterial;
	}
#endif
	
	// Debug Overlay materials use the depth image, so they don't need to use YCbCr conversion.
	const bool bUsesCameraTexture = (OverlayMaterialToUse == RegularOverlayMaterial);

	RenderVideoOverlayWithMaterial(RHICmdList, InView, OverlayMaterialToUse, false, bUsesCameraTexture);
	
#if ALLOWS_DEBUG_OVERLAY
	if (GDebugOverlayMode)
	{
		// Do not draw the occlusion overlay in debug mode
		return;
	}
#endif
	
	if (bEnableOcclusionRendering && DepthOcclusionMaterial)
	{
		UARUtilitiesFunctionLibrary::UpdateWorldToMeterScale(DepthOcclusionMaterial, 100.f);
		RenderVideoOverlayWithMaterial(RHICmdList, InView, DepthOcclusionMaterial, true, true);
	}
}

void FGoogleARCorePassthroughCameraRenderer::UpdateCameraTextures(UTexture* NewCameraTexture, UTexture* DepthTexture, bool bEnableOcclusion)
{
	bEnableOcclusionRendering = DepthTexture ? bEnableOcclusion : false;
	static const auto DepthToMeters = 1.f / 1000.f;
	
	if (DepthTexture)
	{
#if ALLOWS_DEBUG_OVERLAY
		if (DepthColorationMaterial)
		{
			// The value in the depth map of ARCore is in millimetre
			UARUtilitiesFunctionLibrary::UpdateSceneDepthTexture(DepthColorationMaterial, DepthTexture, DepthToMeters);
		}
		
		if (GDebugOverlayMode == (int32)EARCoreDebugOverlayMode::SceneDepthMap)
		{
			// Max out at 5 meters
			UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(DebugOverlayMaterial, DepthTexture, 1.f/5000.f);
		}
#endif
	}
	
	if (bEnableOcclusionRendering && DepthOcclusionMaterial)
	{
		UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(DepthOcclusionMaterial, NewCameraTexture);
		UARUtilitiesFunctionLibrary::UpdateSceneDepthTexture(DepthOcclusionMaterial, DepthTexture, DepthToMeters);
	}
	
	UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(RegularOverlayMaterial, NewCameraTexture);
}

void FGoogleARCorePassthroughCameraRenderer::UpdateCameraYCbCrConversion(IYCbCrConversionQuery* NewYCbCrConversionQuery)
{
	if (NewYCbCrConversionQuery)
	{
		ENQUEUE_RENDER_COMMAND(SetCameraYCbCrConversion)(
			[this, NewYCbCrConversionQuery](FRHICommandListImmediate& RHICmdList)
			{
				CameraYCbCrConversion = NewYCbCrConversionQuery->GetYCbCrConversion_RenderThread();
			});
	}
}

void FGoogleARCorePassthroughCameraRenderer::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(RegularOverlayMaterial);
	Collector.AddReferencedObject(DebugOverlayMaterial);
	Collector.AddReferencedObject(DepthColorationMaterial);
	Collector.AddReferencedObject(DepthOcclusionMaterial);
}
