// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionGoogleARCorePassthroughCamera.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "MaterialCompiler.h"
#include "GoogleARCorePassthroughCameraExternalTextureGuid.h"

UDEPRECATED_MaterialExpressionGoogleARCorePassthroughCamera::UDEPRECATED_MaterialExpressionGoogleARCorePassthroughCamera(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
int32 UDEPRECATED_MaterialExpressionGoogleARCorePassthroughCamera::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->TextureSample(
		Compiler->ExternalTexture(GoogleARCorePassthroughCameraExternalTextureGuid),
		Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false),
		EMaterialSamplerType::SAMPLERTYPE_Color);
}

int32 UDEPRECATED_MaterialExpressionGoogleARCorePassthroughCamera::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return INDEX_NONE;
}

void UDEPRECATED_MaterialExpressionGoogleARCorePassthroughCamera::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("DEPRECATED, USE UARBlueprintLibrary::GetARTexture - GoogleARCore Passthrough Camera"));
}
#endif
