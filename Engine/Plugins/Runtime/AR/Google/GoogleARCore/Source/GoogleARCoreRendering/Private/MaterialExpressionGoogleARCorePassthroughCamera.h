// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionGoogleARCorePassthroughCamera.generated.h"

/**
* Implements a node sampling from the ARCore Passthrough external texture.
*/
UCLASS(collapsecategories, hidecategories = Object, Deprecated)
class GOOGLEARCORERENDERING_API UDEPRECATED_MaterialExpressionGoogleARCorePassthroughCamera : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinates;

	/** only used if Coordinates are not hooked up. */
	UPROPERTY(EditAnywhere, Category = UMaterialExpressionGoogleARCorePassthroughCamera)
	uint32 ConstCoordinate;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	//~ End UMaterialExpression Interface
#endif // WITH_EDITOR
};

