// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GoogleARCoreYCbCrConversion.h"
#include "CoreMinimal.h"
#include "UObject/ConstructorHelpers.h"
#include "RHI.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/Texture.h"
#include "UObject/GCObject.h"

#include "GoogleARCorePassthroughCameraRenderer.generated.h"


class UTexture;
class FRHICommandListImmediate;
class FSceneViewFamily;

/** A helper class that is used to load the GoogleARCorePassthroughCameraMaterial from its default object. */
UCLASS()
class GOOGLEARCORERENDERING_API UGoogleARCoreCameraOverlayMaterialLoader : public UObject
{
	GENERATED_BODY()

public:
	/** A pointer to the camera overlay material that will be used to render the passthrough camera texture as background. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> RegularOverlayMaterial;
	
	/** A pointer to the camera overlay material that will be used to render the passthrough camera texture as background. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DebugOverlayMaterial;
	
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DepthOcclusionMaterial;
	
	/** Material used for rendering the coloration of the depth map. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DepthColorationMaterial;

	UGoogleARCoreCameraOverlayMaterialLoader()
	{
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> RegularOverlayMaterialRef(TEXT("/ARUtilities/Materials/MI_PassthroughCameraExternalTexture.MI_PassthroughCameraExternalTexture"));
		RegularOverlayMaterial = RegularOverlayMaterialRef.Object;
		
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> DebugOverlayMaterialRef(TEXT("/ARUtilities/Materials/M_PassthroughCamera.M_PassthroughCamera"));
		DebugOverlayMaterial = DebugOverlayMaterialRef.Object;
		
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> DepthOcclusionMaterialRef(TEXT("/GoogleARCore/M_SceneDepthOcclusion.M_SceneDepthOcclusion"));
		DepthOcclusionMaterial = DepthOcclusionMaterialRef.Object;
		
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> DepthColorationMaterialRef(TEXT("/ARUtilities/Materials/M_DepthColoration.M_DepthColoration"));
		DepthColorationMaterial = DepthColorationMaterialRef.Object;
	}
};

class GOOGLEARCORERENDERING_API FGoogleARCorePassthroughCameraRenderer : public FGCObject
{
public:
	FGoogleARCorePassthroughCameraRenderer();

	void InitializeRenderer_RenderThread(FSceneViewFamily& InViewFamily);

	void RenderVideoOverlay_RenderThread(FRHICommandList& RHICmdList, FSceneView& InView);
	
	void UpdateCameraTextures(UTexture* NewCameraTexture, UTexture* DepthTexture, bool bEnableOcclusion);

	void UpdateCameraYCbCrConversion(IYCbCrConversionQuery* NewYCbCrConversionQuery);
	
	void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FGoogleARCorePassthroughCameraRenderer");
	}

private:
	void RenderVideoOverlayWithMaterial(FRHICommandList& RHICmdList, FSceneView& InView, UMaterialInstanceDynamic* OverlayMaterialToUse, bool bRenderingOcclusion, bool bUsesCameraTexture);

private:
	FBufferRHIRef OverlayIndexBufferRHI;
	FBufferRHIRef OverlayVertexBufferRHI;
	
	TObjectPtr<UMaterialInstanceDynamic> RegularOverlayMaterial = nullptr;
	TObjectPtr<UMaterialInstanceDynamic> DebugOverlayMaterial = nullptr;
	TObjectPtr<UMaterialInstanceDynamic> DepthColorationMaterial = nullptr;
	TObjectPtr<UMaterialInstanceDynamic> DepthOcclusionMaterial = nullptr;
	
	bool bEnableOcclusionRendering = false;

	FYCbCrConversion CameraYCbCrConversion;
};
