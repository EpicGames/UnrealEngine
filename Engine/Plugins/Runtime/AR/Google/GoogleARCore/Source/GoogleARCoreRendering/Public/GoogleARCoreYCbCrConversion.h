// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EYCbCrModelConversion : uint8
{
	None = 0,
	YCbCrIdentity,
	YCbCrRec709,
	YCbCrRec601,
	YCbCrRec2020,
	Max
};

enum class EYCbCrRange : uint8
{
	Unknown = 0,
	Full,
	Narrow,
	Max
};

struct FYCbCrConversion
{

	EYCbCrModelConversion YCbCrModelConversion = EYCbCrModelConversion::None;
	EYCbCrRange YCbCrRange = EYCbCrRange::Unknown;
	uint8 NumBits = 8;
};

class IYCbCrConversionQuery
{
public:
	virtual ~IYCbCrConversionQuery() = default;
	
	// Returns the YCbCr conversion paramers of Vulkan's external Hardware Buffer.
	// It should be called from the render thread only.
	virtual FYCbCrConversion GetYCbCrConversion_RenderThread() const = 0;
};
